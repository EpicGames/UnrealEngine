// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class FontEditor : ModuleRules
{
	public FontEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AppFramework",
				"AssetRegistry",
				"Core",
				"CoreUObject",
				"ContentBrowser",
				"DesktopPlatform",
				"DesktopWidgets",
				"Engine",
                "InputCore",
				"RenderCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"PropertyEditor",
				"EditorStyle",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"EditorFramework",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"WorkspaceMenuStructure",
				"MainFrame",
			}
		);
	}
}
