// Copyright Epic Games, Inc. All Rights Reserved.

#include "FontEditor.h"

#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/List.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "DesktopPlatformModule.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorReimportHandler.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "Engine/FontImportOptions.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/TextureDefines.h"
#include "Exporters/Exporter.h"
#include "Exporters/TextureExporterTGA.h"
#include "Factories/Factory.h"
#include "Factories/FontFactory.h"
#include "Factories/TextureFactory.h"
#include "Factories/TrueTypeFontFactory.h"
#include "FontEditorModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMisc.h"
#include "IDesktopPlatform.h"
#include "IDetailsView.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "SCompositeFontEditor.h"
#include "SFontEditorViewport.h"
#include "Selection.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Subsystems/ImportSubsystem.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FontEditor"

DEFINE_LOG_CATEGORY_STATIC(LogFontEditor, Log, All);

FString FFontEditor::LastPath;

const FName FFontEditor::TexturePagesViewportTabId( TEXT( "FontEditor_TexturePagesViewport" ) );
const FName FFontEditor::CompositeFontEditorTabId( TEXT( "FontEditor_CompositeFontEditor" ) );
const FName FFontEditor::PreviewTabId( TEXT( "FontEditor_FontPreview" ) );
const FName FFontEditor::PropertiesTabId( TEXT( "FontEditor_FontProperties" ) );
const FName FFontEditor::PagePropertiesTabId( TEXT( "FontEditor_FontPageProperties" ) );

/*-----------------------------------------------------------------------------
   FFontEditorCommands
-----------------------------------------------------------------------------*/

class FFontEditorCommands : public TCommands<FFontEditorCommands>
{
public:
	/** Constructor */
	FFontEditorCommands() 
		: TCommands<FFontEditorCommands>("FontEditor", NSLOCTEXT("Contexts", "FontEditor", "Font Editor"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}
	
	/** Imports a single font page */
	TSharedPtr<FUICommandInfo> Update;
	
	/** Imports all font pages */
	TSharedPtr<FUICommandInfo> UpdateAll;
	
	/** Exports a single font page */
	TSharedPtr<FUICommandInfo> ExportPage;
	
	/** Exports all font pages */
	TSharedPtr<FUICommandInfo> ExportAllPages;

	/** Spawns a color picker for changing the background color of the font preview viewport */
	TSharedPtr<FUICommandInfo> FontBackgroundColor;

	/** Spawns a color picker for changing the foreground color of the font preview viewport */
	TSharedPtr<FUICommandInfo> FontForegroundColor;

	/** Initialize commands */
	virtual void RegisterCommands() override;
};

void FFontEditorCommands::RegisterCommands()
{
	UI_COMMAND(Update, "Update", "Imports a texture to replace the currently selected page.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UpdateAll, "Update All", "Imports a set of textures to replace all pages.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportPage, "Export", "Exports the currently selected page.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportAllPages, "Export All", "Exports all pages.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(FontBackgroundColor, "Background", "Changes the background color of the previewer.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FontForegroundColor, "Foreground", "Changes the foreground color of the previewer.", EUserInterfaceActionType::Button, FInputChord());
}

void FFontEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_FontEditor", "Font Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( TexturePagesViewportTabId, FOnSpawnTab::CreateSP(this, &FFontEditor::SpawnTab_TexturePagesViewport) )
		.SetDisplayName( LOCTEXT("TexturePagesViewportTab", "Texture Pages") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports"))
		.SetMenuType( TAttribute<ETabSpawnerMenuType::Type>::Create(TAttribute<ETabSpawnerMenuType::Type>::FGetter::CreateSP(this, &FFontEditor::GetTabSpawnerMenuType, TexturePagesViewportTabId)) );

	InTabManager->RegisterTabSpawner( CompositeFontEditorTabId, FOnSpawnTab::CreateSP(this, &FFontEditor::SpawnTab_CompositeFontEditor) )
		.SetDisplayName( LOCTEXT("CompositeFontEditorTab", "Composite Font") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Tabs.PageProperties"))
		.SetMenuType( TAttribute<ETabSpawnerMenuType::Type>::Create(TAttribute<ETabSpawnerMenuType::Type>::FGetter::CreateSP(this, &FFontEditor::GetTabSpawnerMenuType, CompositeFontEditorTabId)) );

	InTabManager->RegisterTabSpawner( PreviewTabId,		FOnSpawnTab::CreateSP(this, &FFontEditor::SpawnTab_Preview) )
		.SetDisplayName( LOCTEXT("PreviewTab", "Preview") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Tabs.Preview"));

	InTabManager->RegisterTabSpawner( PropertiesTabId,	FOnSpawnTab::CreateSP(this, &FFontEditor::SpawnTab_Properties) )
		.SetDisplayName( LOCTEXT("PropertiesTabId", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner( PagePropertiesTabId,FOnSpawnTab::CreateSP(this, &FFontEditor::SpawnTab_PageProperties) )
		.SetDisplayName( LOCTEXT("PagePropertiesTab", "Page Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Tabs.PageProperties"))
		.SetMenuType( TAttribute<ETabSpawnerMenuType::Type>::Create(TAttribute<ETabSpawnerMenuType::Type>::FGetter::CreateSP(this, &FFontEditor::GetTabSpawnerMenuType, PagePropertiesTabId)) );
}

void FFontEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( TexturePagesViewportTabId );
	InTabManager->UnregisterTabSpawner( CompositeFontEditorTabId );
	InTabManager->UnregisterTabSpawner( PreviewTabId );	
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
	InTabManager->UnregisterTabSpawner( PagePropertiesTabId );
}

FFontEditor::FFontEditor()
	: Font(nullptr)
	, TGAExporter(nullptr)
	, Factory(nullptr)
{
}

FFontEditor::~FFontEditor()
{
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	if (Editor != NULL)
	{
		Editor->UnregisterForUndo(this);
		Editor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);
	}
}

void FFontEditor::InitFontEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit)
{
	FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FFontEditor::OnPostReimport);

	// Register to be notified when an object is reimported.
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddSP(this, &FFontEditor::OnObjectReimported);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FFontEditor::OnObjectPropertyChanged);

	Font = CastChecked<UFont>(ObjectToEdit);

	// Support undo/redo
	Font->SetFlags(RF_Transactional);
	
	// Create a TGA exporter
	TGAExporter = NewObject<UTextureExporterTGA>();
	// And our importer
	Factory = NewObject<UTextureFactory>();
	// Set the defaults
	Factory->Blending = BLEND_Opaque;
	Factory->ShadingModel = MSM_Unlit;
	Factory->bDeferCompression = true;
	Factory->MipGenSettings = TMGS_NoMipmaps;
	
	UEditorEngine* Editor = (UEditorEngine*)GEngine;
	if (Editor != NULL)
	{
		Editor->RegisterForUndo(this);
	}
	// Register our commands. This will only register them if not previously registered
	FFontEditorCommands::Register();

	BindCommands();

	CreateInternalWidgets();

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_FontEditor_Layout_v4")
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation( Orient_Vertical )
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Horizontal) ->SetSizeCoefficient(0.9f)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical) ->SetSizeCoefficient(0.65f)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.85f)
					->AddTab( TexturePagesViewportTabId, ETabState::OpenedTab )
					->AddTab( CompositeFontEditorTabId, ETabState::OpenedTab )
				)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.15f)
					->AddTab( PreviewTabId, ETabState::OpenedTab )
				)
			)
			->Split
			(
				FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical) ->SetSizeCoefficient(0.35f)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
					->AddTab( PropertiesTabId, ETabState::OpenedTab )
				)
				->Split
				(
					FTabManager::NewStack() ->SetSizeCoefficient(0.5f)
					->AddTab( PagePropertiesTabId, ETabState::OpenedTab )
				)
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FontEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit);

	IFontEditorModule* FontEditorModule = &FModuleManager::LoadModuleChecked<IFontEditorModule>("FontEditor");
	AddMenuExtender(FontEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	UpdateLayout();

	// @todo toolkit world centric editing
	/*if(IsWorldCentricAssetEditor())
	{
		SpawnToolkitTab(GetToolbarTabId(), FString(), EToolkitTabSpot::ToolBar);
		SpawnToolkitTab(TexturePagesViewportTabId, FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(CompositeFontEditorTabId, FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(PreviewTabId, FString(), EToolkitTabSpot::Viewport);
		SpawnToolkitTab(PropertiesTabId, FString(), EToolkitTabSpot::Details);
		SpawnToolkitTab(PagePropertiesTabId, FString(), EToolkitTabSpot::Details);
	}*/
}

UFont* FFontEditor::GetFont() const
{
	return Font;
}

void FFontEditor::SetSelectedPage(int32 PageIdx)
{
	TArray<UObject*> PagePropertyObjects;
	if (Font->Textures.IsValidIndex(PageIdx))
	{
		PagePropertyObjects.Add(Font->Textures[PageIdx]);
	}
	FontPageProperties->SetObjects(PagePropertyObjects);
}

FName FFontEditor::GetToolkitFName() const
{
	return FName("FontEditor");
}

FText FFontEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Font Editor" );
}

FString FFontEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Font ").ToString();
}

FLinearColor FFontEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

TSharedRef<SDockTab> FFontEditor::SpawnTab_TexturePagesViewport( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == TexturePagesViewportTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("TexturePagesViewportTitle", "Texture Pages"))
		[
			FontViewport.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

TSharedRef<SDockTab> FFontEditor::SpawnTab_CompositeFontEditor( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == CompositeFontEditorTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("CompositeFontEditorTitle", "Composite Font"))
		[
			CompositeFontEditor.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

TSharedRef<SDockTab> FFontEditor::SpawnTab_Preview( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == PreviewTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("FontPreviewTitle", "Preview"))
		[
			FontPreview.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

TSharedRef<SDockTab> FFontEditor::SpawnTab_Properties( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == PropertiesTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("FontPropertiesTitle", "Details"))
		[
			FontProperties.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

TSharedRef<SDockTab> FFontEditor::SpawnTab_PageProperties( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == PagePropertiesTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("FontPagePropertiesTitle", "Page Details"))
		[
			FontPageProperties.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

void FFontEditor::AddToSpawnedToolPanels( const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab )
{
	TWeakPtr<SDockTab>* TabSpot = SpawnedToolPanels.Find(TabIdentifier);
	if (!TabSpot)
	{
		SpawnedToolPanels.Add(TabIdentifier, SpawnedTab);
	}
	else
	{
		check(!TabSpot->IsValid());
		*TabSpot = SpawnedTab;
	}
}

void FFontEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Font);
	Collector.AddReferencedObject(TGAExporter);
	Collector.AddReferencedObject(Factory);
}

void FFontEditor::OnPreviewTextChanged(const FText& Text)
{
	FontPreviewWidget->SetPreviewText(Text);
}

TOptional<float> FFontEditor::GetDrawFontScale() const
{
	return FontPreviewWidget->GetPreviewFontScale();
}

void FFontEditor::OnDrawFontScaleChanged(float InNewValue, ETextCommit::Type CommitType)
{
	FontPreviewWidget->SetPreviewFontScale(InNewValue);
}

ECheckBoxState FFontEditor::GetDrawFontMetricsState() const
{
	return (FontPreviewWidget->GetPreviewFontMetrics()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FFontEditor::OnDrawFontMetricsStateChanged(ECheckBoxState NewState)
{
	FontPreviewWidget->SetPreviewFontMetrics(NewState == ECheckBoxState::Checked);
}

void FFontEditor::PostUndo(bool bSuccess)
{
	// Make sure we're using the correct layout, as the undo/redo may have changed the font cache type property
	UpdateLayout();

	CompositeFontEditor->Refresh();
}

void FFontEditor::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{
	static const FName FontCacheTypePropertyName = GET_MEMBER_NAME_CHECKED(UFont, FontCacheType);
	static const FName CompositeFontPropertyName = UFont::GetInternalCompositeFontPropertyName();
	static const FName TexturePageWidthName = GET_MEMBER_NAME_CHECKED(FFontImportOptionsData, TexturePageWidth);
	static const FName TexturePageMaxHeightName = GET_MEMBER_NAME_CHECKED(FFontImportOptionsData, TexturePageMaxHeight);
	static const FName DistanceFieldScaleFactorName = GET_MEMBER_NAME_CHECKED(FFontImportOptionsData, DistanceFieldScaleFactor);

	if(PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == FontCacheTypePropertyName)
	{
		// Show a warning message, as what we're about to do will destroy any existing data in this font object
		const EAppReturnType::Type DlgResult = FMessageDialog::Open(
			EAppMsgType::YesNo, 
			LOCTEXT("ChangeCacheTypeWarningMsg", "Changing the cache type will cause this font to be reinitialized (discarding any existing data).\n\nAre you sure you want to proceed?"), 
			LOCTEXT("ChangeCacheTypeWarningTitle", "Really change the font cache type?")
			);

		bool bSuccessfullyChangedCacheType = false;
		if(DlgResult == EAppReturnType::Yes)
		{
			bSuccessfullyChangedCacheType = RecreateFontObject(Font->FontCacheType);
		}

		if(bSuccessfullyChangedCacheType)
		{
			CompositeFontEditor->Refresh();

			// If we changed the font cache type, then we need to update the UI to hide the invalid tabs and spawn the new ones
			UpdateLayout();
			FontProperties->ForceRefresh();
		}
		else
		{
			// Restore the old font cache type
			switch(Font->FontCacheType)
			{
			case EFontCacheType::Offline:
				Font->FontCacheType = EFontCacheType::Runtime;
				break;

			case EFontCacheType::Runtime:
				Font->FontCacheType = EFontCacheType::Offline;
				break;

			default:
				break;
			}
		}
	}

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == DistanceFieldScaleFactorName)
	{
		const uint32 SignedInt32NumBits = 31;
		const uint32 Log2TexturePageWidth = FMath::CeilLogTwo(Font->ImportOptions.TexturePageWidth);
		const uint32 Log2TexturePageMaxHeight = FMath::CeilLogTwo(Font->ImportOptions.TexturePageMaxHeight);
		const uint32 Log2BytesPerPixel = 2;

		const int32 MaxDistanceFieldScaleFactor = 1 << ((SignedInt32NumBits - Log2BytesPerPixel - Log2TexturePageWidth - Log2TexturePageMaxHeight) / 2);
		if (Font->ImportOptions.DistanceFieldScaleFactor > MaxDistanceFieldScaleFactor)
		{
			Font->ImportOptions.DistanceFieldScaleFactor = MaxDistanceFieldScaleFactor;
		}
	}

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TexturePageWidthName)
	{
		const uint32 SignedInt32NumBits = 31;
		const uint32 Log2DistanceFieldScaleFactor = FMath::Max(1U, FMath::CeilLogTwo(Font->ImportOptions.DistanceFieldScaleFactor));
		const uint32 Log2TexturePageMaxHeight = FMath::CeilLogTwo(Font->ImportOptions.TexturePageMaxHeight);
		const uint32 Log2BytesPerPixel = 2;

		const int32 MaxTexturePageWidth = 1 << (SignedInt32NumBits - Log2BytesPerPixel - 2 * Log2DistanceFieldScaleFactor - Log2TexturePageMaxHeight);
		if (Font->ImportOptions.TexturePageWidth > MaxTexturePageWidth)
		{
			Font->ImportOptions.TexturePageWidth = MaxTexturePageWidth;
		}
	}

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == TexturePageMaxHeightName)
	{
		const uint32 SignedInt32NumBits = 31;
		const uint32 Log2DistanceFieldScaleFactor = FMath::Max(1U, FMath::CeilLogTwo(Font->ImportOptions.DistanceFieldScaleFactor));
		const uint32 Log2TexturePageWidth = FMath::CeilLogTwo(Font->ImportOptions.TexturePageWidth);
		const uint32 Log2BytesPerPixel = 2;

		const int32 MaxTexturePageMaxHeight = 1 << (SignedInt32NumBits - Log2BytesPerPixel - 2 * Log2DistanceFieldScaleFactor - Log2TexturePageWidth);
		if (Font->ImportOptions.TexturePageMaxHeight > MaxTexturePageMaxHeight)
		{
			Font->ImportOptions.TexturePageMaxHeight = MaxTexturePageMaxHeight;
		}
	}

	// If we changed a property of the composite font, we need to refresh the composite font editor
	if(PropertyThatChanged && PropertyThatChanged->GetHead()->GetValue()->GetFName() == CompositeFontPropertyName)
	{
		CompositeFontEditor->Refresh();
	}

	if(Font->FontCacheType == EFontCacheType::Offline)
	{
		FontViewport->RefreshViewport();
	}

	FontPreviewWidget->RefreshViewport();
}

void FFontEditor::UpdateLayout()
{
	if(CurrentEditorLayout.IsSet() && CurrentEditorLayout.GetValue() == Font->FontCacheType)
	{
		return;
	}

	auto CloseTab = [this](const FName& TabName)
	{
		TWeakPtr<SDockTab>* const FoundExistingTab = SpawnedToolPanels.Find(TabName);
		if(FoundExistingTab)
		{
			TSharedPtr<SDockTab> ExistingTab = FoundExistingTab->Pin();
			if(ExistingTab.IsValid())
			{
				ExistingTab->RequestCloseTab();
			}
		}
	};

	switch(Font->FontCacheType)
	{
	case EFontCacheType::Offline:
		TabManager->TryInvokeTab(TexturePagesViewportTabId);
		TabManager->TryInvokeTab(PagePropertiesTabId);
		CloseTab(CompositeFontEditorTabId);
		break;

	case EFontCacheType::Runtime:
		TabManager->TryInvokeTab(CompositeFontEditorTabId);
		CloseTab(TexturePagesViewportTabId);
		CloseTab(PagePropertiesTabId);
		break;

	default:
		break;
	}

	CurrentEditorLayout = Font->FontCacheType;
}

ETabSpawnerMenuType::Type FFontEditor::GetTabSpawnerMenuType( FName InTabName ) const
{
	if( (Font->FontCacheType == EFontCacheType::Offline && (InTabName == CompositeFontEditorTabId)) ||
		(Font->FontCacheType == EFontCacheType::Runtime && (InTabName == TexturePagesViewportTabId || InTabName == PagePropertiesTabId)))
	{
		return ETabSpawnerMenuType::Hidden;
	}

	return ETabSpawnerMenuType::Enabled;
}

void FFontEditor::CreateInternalWidgets()
{
	FontViewport = 
	SNew(SFontEditorViewport)
	.FontEditor(SharedThis(this));

	CompositeFontEditor = 
	SNew(SCompositeFontEditor)
	.FontEditor(SharedThis(this));

	FontPreview =
	SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.FillHeight(1.0f)
	.Padding(0.0f, 0.0f, 0.0f, 4.0f)
	[
		SAssignNew(FontPreviewWidget, SFontEditorViewport)
		.FontEditor(SharedThis(this))
		.IsPreview(true)
	]
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		[
			SAssignNew(FontPreviewText, SEditableTextBox)
			.Text(LOCTEXT("DefaultPreviewText", "The quick brown fox jumps over the lazy dog"))
			.SelectAllTextWhenFocused(true)
			.OnTextChanged(this, &FFontEditor::OnPreviewTextChanged)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SNumericEntryBox<float>)
			.Value(this, &FFontEditor::GetDrawFontScale)
			.MinValue(1.f)
			.MaxValue(10.f)
			.OnValueCommitted(this, &FFontEditor::OnDrawFontScaleChanged)
		]


		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(FMargin(2.0f, 0.0f, 0.0f, 0.0f))
		[
			SNew(SCheckBox)
			.IsChecked(this, &FFontEditor::GetDrawFontMetricsState)
			.OnCheckStateChanged(this, &FFontEditor::OnDrawFontMetricsStateChanged)
			.ToolTipText(LOCTEXT("DrawFontMetricsToolTip", "Draw the font metrics (line height, glyph bounding boxes, and base-line) as part of the preview?"))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DrawFontMetricsLabel", "Draw Font Metrics"))
			]
		]
	];

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FontProperties = PropertyModule.CreateDetailView(Args);
	FontPageProperties = PropertyModule.CreateDetailView(Args);

	FontProperties->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateRaw(this, &FFontEditor::GetIsPropertyVisible));
	FontProperties->SetObject( Font );
}

void FFontEditor::ExtendToolbar()
{
	struct Local
	{
		static void FillToolbar(FToolBarBuilder& ToolbarBuilder)
		{
			ToolbarBuilder.BeginSection("FontImportExport");
			{
				ToolbarBuilder.AddToolBarButton(FFontEditorCommands::Get().Update);
				ToolbarBuilder.AddToolBarButton(FFontEditorCommands::Get().UpdateAll);
				ToolbarBuilder.AddToolBarButton(FFontEditorCommands::Get().ExportPage);
				ToolbarBuilder.AddToolBarButton(FFontEditorCommands::Get().ExportAllPages);
			}
			ToolbarBuilder.EndSection();

			ToolbarBuilder.BeginSection("FontPreviewer");
			{
				ToolbarBuilder.AddToolBarButton(FFontEditorCommands::Get().FontBackgroundColor);
				ToolbarBuilder.AddToolBarButton(FFontEditorCommands::Get().FontForegroundColor);
			}
			ToolbarBuilder.EndSection();
		}
	};

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic( &Local::FillToolbar )
		);

	AddToolbarExtender(ToolbarExtender);
	// AddToSpawnedToolPanels( GetToolbarTabId(), ToolbarTab );

	IFontEditorModule* FontEditorModule = &FModuleManager::LoadModuleChecked<IFontEditorModule>("FontEditor");
	AddToolbarExtender(FontEditorModule->GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

void FFontEditor::BindCommands()
{
	const FFontEditorCommands& Commands = FFontEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.Update,
		FExecuteAction::CreateSP(this, &FFontEditor::OnUpdate),
		FCanExecuteAction::CreateSP(this, &FFontEditor::OnUpdateEnabled));

	ToolkitCommands->MapAction(
		Commands.UpdateAll,
		FExecuteAction::CreateSP(this, &FFontEditor::OnUpdateAll),
		FCanExecuteAction::CreateSP(this, &FFontEditor::OnUpdateAllEnabled));

	ToolkitCommands->MapAction(
		Commands.ExportPage,
		FExecuteAction::CreateSP(this, &FFontEditor::OnExport),
		FCanExecuteAction::CreateSP(this, &FFontEditor::OnExportEnabled));

	ToolkitCommands->MapAction(
		Commands.ExportAllPages,
		FExecuteAction::CreateSP(this, &FFontEditor::OnExportAll),
		FCanExecuteAction::CreateSP(this, &FFontEditor::OnExportAllEnabled));

	ToolkitCommands->MapAction(
		Commands.FontBackgroundColor,
		FExecuteAction::CreateSP(this, &FFontEditor::OnBackgroundColor),
		FCanExecuteAction::CreateSP(this, &FFontEditor::OnBackgroundColorEnabled));

	ToolkitCommands->MapAction(
		Commands.FontForegroundColor,
		FExecuteAction::CreateSP(this, &FFontEditor::OnForegroundColor),
		FCanExecuteAction::CreateSP(this, &FFontEditor::OnForegroundColorEnabled));
}

void FFontEditor::OnUpdate()
{
	int32 CurrentSelectedPage = FontViewport->GetCurrentSelectedPage();

	if (CurrentSelectedPage > INDEX_NONE)
	{
		TArray<FString> OpenFilenames;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		bool bOpened = false;
		if ( DesktopPlatform )
		{
			bOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				LOCTEXT("ImportDialogTitle", "Import").ToString(),
				LastPath,
				TEXT(""),
				TEXT("TGA Files (*.tga)|*.tga"),
				EFileDialogFlags::None,
				OpenFilenames
				);
		}

		if (bOpened)
		{
			LastPath = FPaths::GetPath(OpenFilenames[0]);
			// Use the common routine for importing the texture
			if (ImportPage(CurrentSelectedPage, *OpenFilenames[0]) == false)
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("CurrentPageNumber"), CurrentSelectedPage );
				Args.Add( TEXT("Filename"), FText::FromString( OpenFilenames[0] ) );

				// Show an error to the user
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("FailedToUpdateFontPage", "Failed to update the font page ({CurrentPageNumber}) with texture ({Filename})"), Args ) );
			}
		}

		GEditor->GetSelectedObjects()->DeselectAll();
		GEditor->GetSelectedObjects()->Select(Font->Textures[CurrentSelectedPage]);

		FontViewport->RefreshViewport();
		FontPreviewWidget->RefreshViewport();
	}
}

bool FFontEditor::OnUpdateEnabled() const
{
	return Font->FontCacheType == EFontCacheType::Offline && FontViewport->GetCurrentSelectedPage() != INDEX_NONE;
}

void FFontEditor::OnUpdateAll()
{
	int32 CurrentSelectedPage = FontViewport->GetCurrentSelectedPage();

	// Open dialog so user can chose which directory to export to
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		FString FolderName;
		const FString Title = FText::Format( NSLOCTEXT("UnrealEd", "Save_F", "Save: {0}"), FText::FromString(Font->GetName()) ).ToString();
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			Title,
			LastPath,
			FolderName
			);

		if ( bFolderSelected )
		{
			LastPath = FolderName;
		// Try to import each file into the corresponding page
		for (int32 Index = 0; Index < Font->Textures.Num(); ++Index)
		{
			// Create a name for the file based off of the font name and page number
			FString FileName = FString::Printf(TEXT("%s/%s_Page_%d.tga"), *LastPath, *Font->GetName(), Index);
			if (ImportPage(Index, *FileName) == false)
			{
				FFormatNamedArguments Args;
				Args.Add( TEXT("CurrentPageNumber"), Index );
				Args.Add( TEXT("Filename"), FText::FromString( FileName ) );

				// Show an error to the user
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("FailedToUpdateFontPage", "Failed to update the font page ({CurrentPageNumber}) with texture ({Filename})"), Args ) );
			}
		}
	}
	}

	GEditor->GetSelectedObjects()->DeselectAll();
	if (CurrentSelectedPage != INDEX_NONE)
	{
		GEditor->GetSelectedObjects()->Select(Font->Textures[CurrentSelectedPage]);
	}

	FontViewport->RefreshViewport();
	FontPreviewWidget->RefreshViewport();
}

bool FFontEditor::OnUpdateAllEnabled() const
{
	return Font->FontCacheType == EFontCacheType::Offline;
}

void FFontEditor::OnExport()
{
	int32 CurrentSelectedPage = FontViewport->GetCurrentSelectedPage();
	
	if (CurrentSelectedPage > INDEX_NONE)
	{
		// Open dialog so user can chose which directory to export to
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if ( DesktopPlatform )
		{
			FString FolderName;
			const FString Title = FText::Format( NSLOCTEXT("UnrealEd", "Save_F", "Save: {0}"), FText::FromString(Font->GetName()) ).ToString();
			const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				Title,
				LastPath,
				FolderName
				);

			if ( bFolderSelected )
			{
				LastPath = FolderName;
			// Create a name for the file based off of the font name and page number
			FString FileName = FString::Printf(TEXT("%s/%s_Page_%d.tga"), *LastPath, *Font->GetName(), CurrentSelectedPage);
			
			// Create that file with the texture data
			UExporter::ExportToFile(Font->Textures[CurrentSelectedPage], TGAExporter, *FileName, false);
		}
		}
	}
}

bool FFontEditor::OnExportEnabled() const
{
	return Font->FontCacheType == EFontCacheType::Offline && FontViewport->GetCurrentSelectedPage() != INDEX_NONE;
}

void FFontEditor::OnExportAll()
{
	// Open dialog so user can chose which directory to export to
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		FString FolderName;
		const FString Title = FText::Format( NSLOCTEXT("UnrealEd", "Save_F", "Save: {0}"), FText::FromString(Font->GetName()) ).ToString();
		const bool bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			Title,
			LastPath,
			FolderName
			);
	
		if ( bFolderSelected )
		{
			LastPath = FolderName;
		// Loop through exporting each file to the specified directory
		for (int32 Index = 0; Index < Font->Textures.Num(); ++Index)
		{
			// Create a name for the file based off of the font name and page number
			FString FileName = FString::Printf(TEXT("%s/%s_Page_%d.tga"), *LastPath, *Font->GetName(), Index);

			// Create that file with the texture data
			UExporter::ExportToFile(Font->Textures[Index], TGAExporter, *FileName, false);
		}
	}
	}
}

bool FFontEditor::OnExportAllEnabled() const
{
	return Font->FontCacheType == EFontCacheType::Offline;
}

void FFontEditor::OnBackgroundColor()
{
	TWeakPtr<SFontEditorViewport> WeakFontPreviewWidget = FontPreviewWidget;

	FColorPickerArgs PickerArgs;
	PickerArgs.bIsModal = true;
	PickerArgs.ParentWidget = FontPreview;
	PickerArgs.bUseAlpha = true;
	PickerArgs.bClampValue = true;
	PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );
	PickerArgs.InitialColor = FontPreviewWidget->GetPreviewBackgroundColor();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([WeakFontPreviewWidget](FLinearColor NewValue)
		{
			if (TSharedPtr<SFontEditorViewport> PinnedWidget = WeakFontPreviewWidget.Pin())
			{
				PinnedWidget->SetPreviewBackgroundColor(NewValue.ToFColorSRGB());
			}
		});

	OpenColorPicker(PickerArgs);
}

bool FFontEditor::OnBackgroundColorEnabled() const
{
	const TWeakPtr<SDockTab>* PreviewTab = SpawnedToolPanels.Find( PreviewTabId );
	return PreviewTab && PreviewTab->IsValid();
}

void FFontEditor::OnForegroundColor()
{
	TWeakPtr<SFontEditorViewport> WeakFontPreviewWidget = FontPreviewWidget;

	FColorPickerArgs PickerArgs;
	PickerArgs.bIsModal = true;
	PickerArgs.ParentWidget = FontPreview;
	PickerArgs.bUseAlpha = true;
	PickerArgs.DisplayGamma = TAttribute<float>::Create( TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma) );
	PickerArgs.InitialColor = FontPreviewWidget->GetPreviewForegroundColor();
	PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateLambda([WeakFontPreviewWidget](FLinearColor NewValue)
		{
			if (TSharedPtr<SFontEditorViewport> PinnedWidget = WeakFontPreviewWidget.Pin())
			{
				PinnedWidget->SetPreviewForegroundColor(NewValue.ToFColorSRGB());
			}
		});

	OpenColorPicker(PickerArgs);
}

bool FFontEditor::OnForegroundColorEnabled() const
{
	const TWeakPtr<SDockTab>* PreviewTab = SpawnedToolPanels.Find( PreviewTabId );
	return PreviewTab && PreviewTab->IsValid();
}

void FFontEditor::OnPostReimport(UObject* InObject, bool bSuccess)
{
	// Ignore if this is regarding a different object
	if ( InObject != Font )
	{
		return;
	}

	if ( bSuccess )
	{
		FontViewport->RefreshViewport();
		FontPreviewWidget->RefreshViewport();
	}
}

void FFontEditor::OnObjectPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (Cast<UFont>(InObject))
	{
		//Force all texts using a font to be refreshed.
		FSlateApplicationBase::Get().InvalidateAllWidgets(false);
		GSlateLayoutGeneration++;
	}

	if (Cast<UFontFace>(InObject))
	{
		// Refresh the composite font editor when a font face is changed as it may affect our preview
		CompositeFontEditor->Refresh();
	}
}

bool FFontEditor::ImportPage(int32 PageNum, const TCHAR* FileName)
{
	bool bSuccess = false;
	TArray<uint8> Data;
	
	// Read the file into an array
	if (FFileHelper::LoadFileToArray(Data, FileName))
	{
		// Make a const pointer for the API to be happy
		const uint8* DataPtr = Data.GetData();
		
		// Create the new texture... note RF_Public because font textures can be referenced directly by material expressions
		UTexture2D* NewPage = (UTexture2D*)Factory->FactoryCreateBinary(UTexture2D::StaticClass(), Font, NAME_None, RF_Public, NULL, TEXT("TGA"), DataPtr, DataPtr + Data.Num(), GWarn);

		if (NewPage != NULL && Font->Textures.IsValidIndex(PageNum))
		{
			UTexture2D* Texture = Font->Textures[PageNum];
			
			// Make sure the sizes are the same
			if (Texture->Source.GetSizeX() == NewPage->Source.GetSizeX() && Texture->Source.GetSizeY() == NewPage->Source.GetSizeY())
			{
				// Set the new texture's settings to match the old texture
				NewPage->CompressionNoAlpha = Texture->CompressionNoAlpha;
				NewPage->CompressionNone = Texture->CompressionNone;
				NewPage->MipGenSettings = Texture->MipGenSettings;
				NewPage->CompressionNoAlpha = Texture->CompressionNoAlpha;
				NewPage->NeverStream = Texture->NeverStream;
				NewPage->CompressionSettings = Texture->CompressionSettings;
				NewPage->Filter = Texture->Filter;
				
				// Now compress the texture
				NewPage->PostEditChange();
				
				// Replace the existing texture with the new one
				Font->Textures[PageNum] = NewPage;
				
				// Dirty the font's package and refresh the content browser to indicate the font's package needs to be saved post-update
				Font->MarkPackageDirty();
			}
			else
			{
				// Tell the user the sizes mismatch
				FMessageDialog::Open( EAppMsgType::Ok, FText::Format( LOCTEXT("UpdateDoesNotMatch", "The updated image ({0}) does not match the original's size"), FText::FromString( FileName ) ) );
			}

			bSuccess = true;
		}
		else if (!Font->Textures.IsValidIndex(PageNum))
		{
			FMessageDialog::Open( EAppMsgType::Ok, LOCTEXT("FailedToImportFontPage", "Tried to import an invalid page number."));
		}
	}

	return bSuccess;
}

void FFontEditor::OnObjectReimported(UObject* InObject)
{
	// Make sure we are using the object that is being reimported, otherwise a lot of needless work could occur.
	if(Font == InObject)
	{
		Font = Cast<UFont>(InObject);

		TArray< UObject* > ObjectList;
		ObjectList.Add(InObject);
		FontProperties->SetObjects(ObjectList);
	}
}

bool FFontEditor::RecreateFontObject(const EFontCacheType NewCacheType)
{
	bool bSuccess = false;

	UFactory* FontFactoryPtr = nullptr;

	switch(NewCacheType)
	{
	case EFontCacheType::Offline:
		// UTrueTypeFontFactory will create a new font object using a texture generated from a user-selection font
		FontFactoryPtr = NewObject<UTrueTypeFontFactory>();
		break;

	case EFontCacheType::Runtime:
		// UFontFactory will create an empty font ready to add new font files to
		FontFactoryPtr = NewObject<UFontFactory>();
		break;

	default:
		break;
	}

	if(FontFactoryPtr && FontFactoryPtr->ConfigureProperties())
	{
		bool OutCanceled = false;
		if (FontFactoryPtr->ImportObject(Font->GetClass(), Font->GetOuter(), *Font->GetName(), RF_Public | RF_Standalone, TEXT(""), nullptr, OutCanceled) != nullptr)
		{
			bSuccess = true;
		}
	}

	if(bSuccess)
	{
		Font->PostEditChange();
		GEditor->BroadcastObjectReimported(Font);
	}

	// Let listeners know whether the reimport was successful or not
	FReimportManager::Instance()->OnPostReimport().Broadcast(Font, bSuccess);

	return bSuccess;
}

bool FFontEditor::GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	static const FName CategoryFName = "Category";
	const FString& CategoryValue = PropertyAndParent.Property.GetMetaData(CategoryFName);

	// We need to hide the properties associated with the category that we're not currently using (either Offline or Runtime)
	const FString CategoryToExclude = (Font->FontCacheType == EFontCacheType::Offline) ? TEXT("RuntimeFont") : TEXT("OfflineFont");

	// We need to hide the properties associated with the category that we're not currently using (either Offline or Runtime)
	return CategoryValue != CategoryToExclude;
}

bool FFontEditor::ShouldPromptForNewFilesOnReload(const UObject& EditingObject) const
{
	return false;
}

void FFontEditor::RefreshPreview()
{
	FontPreviewWidget->RefreshViewport();
}

#undef LOCTEXT_NAMESPACE
