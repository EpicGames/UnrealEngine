// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "EditorUndoClient.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "IFontEditor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

class FReferenceCollector;
class SCompositeFontEditor;
class SDockTab;
class SEditableTextBox;
class SFontEditorViewport;
class SVerticalBox;
class UFont;
class UTextureExporterTGA;
class UTextureFactory;
struct FPropertyChangedEvent;

enum class EFontCacheType : uint8;

/*-----------------------------------------------------------------------------
   FFontEditor
-----------------------------------------------------------------------------*/

class FFontEditor : public IFontEditor, public FGCObject, public FNotifyHook, public FEditorUndoClient
{
public:
	FFontEditor();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Destructor */
	virtual ~FFontEditor();

	/** Edits the specified Font object */
	void InitFontEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit);

	/** IFontEditor interface */
	virtual UFont* GetFont() const override;
	virtual void SetSelectedPage(int32 PageIdx) override;
	virtual void RefreshPreview() override;
	
	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FFontEditor");
	}

	/** Called to determine if the user should be prompted for a new file if one is missing during an asset reload*/
	virtual bool ShouldPromptForNewFilesOnReload(const UObject& object) const override;

protected:
	/** Called when the preview text changes */
	void OnPreviewTextChanged(const FText& Text);
	
	/** Called to handle the "Draw Font Metrics" check box */
	ECheckBoxState GetDrawFontMetricsState() const;
	void OnDrawFontMetricsStateChanged(ECheckBoxState NewState);

	/** Called to handle the "Draw Font Scale" numeric entry box */
	TOptional<float> GetDrawFontScale() const;
	void OnDrawFontScaleChanged(float InNewValue, ETextCommit::Type CommitType);

	//~ Begin FEditorUndoClient Interface
	/** Handles any post undo cleanup of the GUI so that we don't have stale data being displayed. */
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	/** FNotifyHook interface */
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

	/** Update the font editor UI based on the type of font being edited */
	void UpdateLayout();

	/** Get the menu type to use for the given tab spawner */
	ETabSpawnerMenuType::Type GetTabSpawnerMenuType( FName InTabName ) const;

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	/** Builds the toolbar widget for the font editor */
	void ExtendToolbar();
	
	/**	Binds our UI commands to delegates */
	void BindCommands();

	/** Toolbar command methods */
	void OnUpdate();
	bool OnUpdateEnabled() const;
	void OnUpdateAll();
	bool OnUpdateAllEnabled() const;
	void OnExport();
	bool OnExportEnabled() const;
	void OnExportAll();
	bool OnExportAllEnabled() const;
	void OnBackgroundColor();
	bool OnBackgroundColorEnabled() const;
	void OnForegroundColor();
	bool OnForegroundColorEnabled() const;
	void OnPostReimport(UObject* InObject, bool bSuccess);
	void OnObjectPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);

	/** Common method for replacing a font page with a new texture */
	bool ImportPage(int32 PageNum, const TCHAR* FileName);

	/**	Spawns the text pages viewport tab */
	TSharedRef<SDockTab> SpawnTab_TexturePagesViewport( const FSpawnTabArgs& Args );

	/**	Spawns the composite font editor UI */
	TSharedRef<SDockTab> SpawnTab_CompositeFontEditor( const FSpawnTabArgs& Args );

	/**	Spawns the preview tab */
	TSharedRef<SDockTab> SpawnTab_Preview( const FSpawnTabArgs& Args );

	/**	Spawns the properties tab */
	TSharedRef<SDockTab> SpawnTab_Properties( const FSpawnTabArgs& Args );

	/**	Spawns the page properties tab */
	TSharedRef<SDockTab> SpawnTab_PageProperties( const FSpawnTabArgs& Args );

	/**	Caches the specified tab for later retrieval */
	void AddToSpawnedToolPanels( const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab );

	/** Callback when an object is reimported, handles steps needed to keep the editor up-to-date. */
	void OnObjectReimported(UObject* InObject);

	/** Recreate the font object so that it's using the given caching method */
	bool RecreateFontObject(const EFontCacheType NewCacheType);

	/** Check to see if the given property should be visible in the details panel */
	bool GetIsPropertyVisible(const struct FPropertyAndParent& PropertyAndParent) const;

private:
	/** The font asset being inspected */
	TObjectPtr<UFont> Font;

	/** List of open tool panels; used to ensure only one exists at any one time */
	TMap< FName, TWeakPtr<SDockTab> > SpawnedToolPanels;

	/** Viewport */
	TSharedPtr<SFontEditorViewport> FontViewport;

	/** Composite font editor UI */
	TSharedPtr<SCompositeFontEditor> CompositeFontEditor;

	/** Preview tab */
	TSharedPtr<SVerticalBox> FontPreview;

	/** Properties tab */
	TSharedPtr<class IDetailsView> FontProperties;

	/** Page properties tab */
	TSharedPtr<class IDetailsView> FontPageProperties;

	/** Preview viewport widget */
	TSharedPtr<SFontEditorViewport> FontPreviewWidget;

	/** Preview text */
	TSharedPtr<SEditableTextBox> FontPreviewText;

	/** The last path exported to or from */
	static FString LastPath;
	
	/** The exporter to use for all font page exporting */
	TObjectPtr<UTextureExporterTGA> TGAExporter;

	/** The factory to create updated pages with */
	TObjectPtr<UTextureFactory> Factory;

	/** The current font editor layout (if any) */
	TOptional<EFontCacheType> CurrentEditorLayout;

	/**	The tab ids for the font editor */
	static const FName TexturePagesViewportTabId;
	static const FName CompositeFontEditorTabId;
	static const FName PreviewTabId;
	static const FName PropertiesTabId;
	static const FName PagePropertiesTabId;
};
