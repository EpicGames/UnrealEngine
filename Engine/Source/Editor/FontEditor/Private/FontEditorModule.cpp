// Copyright Epic Games, Inc. All Rights Reserved.

#include "FontEditorModule.h"

#include "Delegates/Delegate.h"
#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "FontEditor.h"
#include "FontFaceEditor.h"
#include "FontFaceDetailsCustomization.h"
#include "HAL/Platform.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "UObject/Class.h"
#include "UObject/UObjectBase.h"

class IToolkitHost;

const FName FontEditorAppIdentifier = FName(TEXT("FontEditorApp"));


/*-----------------------------------------------------------------------------
   FFontEditorModule
-----------------------------------------------------------------------------*/

class FFontEditorModule : public IFontEditorModule
{
public:
	/** Constructor, set up console commands and variables **/
	FFontEditorModule()
	{
	}

	/** Called right after the module DLL has been loaded and the module object has been created */
	virtual void StartupModule() override
	{
		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UFontFace::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FFontFaceDetailsCustomization::MakeInstance));
	}

	/** Called before the module is unloaded, right before the module object is destroyed */
	virtual void ShutdownModule() override
	{
		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

			if (UObjectInitialized())
			{
				PropertyModule.UnregisterCustomClassLayout(UFontFace::StaticClass()->GetFName());
			}
		}
	}

	/** Creates a new Font editor */
	virtual TSharedRef<IFontEditor> CreateFontEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UFont* Font ) override
	{
		TSharedRef<FFontEditor> NewFontEditor(new FFontEditor());
		NewFontEditor->InitFontEditor(Mode, InitToolkitHost, Font);
		return NewFontEditor;
	}

	/** Creates a new Font Face editor */
	virtual TSharedRef<IFontFaceEditor> CreateFontFaceEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UFontFace* FontFace ) override
	{
		TSharedRef<FFontFaceEditor> NewFontEditor(new FFontFaceEditor());
		NewFontEditor->InitFontFaceEditor(Mode, InitToolkitHost, FontFace);
		return NewFontEditor;
	}

	/** Gets the extensibility managers for outside entities to extend static mesh editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

private:
	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};

IMPLEMENT_MODULE( FFontEditorModule, FontEditor );
