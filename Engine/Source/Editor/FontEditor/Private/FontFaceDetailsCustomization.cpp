// Copyright Epic Games, Inc. All Rights Reserved.

#include "FontFaceDetailsCustomization.h"

#include "DesktopPlatformModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "EditorDirectories.h"
#include "EditorFontGlyphs.h"
#include "Engine/FontFace.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/FontCache.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/Platform.h"
#include "IDesktopPlatform.h"
#include "IDetailGroup.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/FileHelper.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "PropertyHandle.h"
#include "Rendering/SlateRenderer.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateEnums.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FontFaceDetailsCustomization"

namespace Private
{
	static const FString FileTypes = TEXT("All Font Files (*.ttf, *.ttc, *.otf, *.otc)|*.ttf;*.ttc;*.otf;*.otc|TrueType fonts (*.ttf, *.ttc)|*.ttf;*.ttc|OpenType fonts (*.otf, *.otc)|*.otf;*.otc");
}
void FFontFaceDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingEdited);

	TSharedPtr<IPropertyHandle> SourceFilenamePropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, SourceFilename));
	SourceFilenamePropertyHandle->MarkHiddenByCustomization();

	IDetailCategoryBuilder& FontFaceCategory = DetailBuilder.EditCategory("FontFace");

	// Source Filename
	{
		FontFaceCategory.AddCustomRow(SourceFilenamePropertyHandle->GetPropertyDisplayName())
			.NameContent()
			[
				SourceFilenamePropertyHandle->CreatePropertyNameWidget()
			]
			.ValueContent()
			.VAlign(VAlign_Center)
			.MinDesiredWidth(TOptional<float>())
			.MaxDesiredWidth(TOptional<float>())
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Font(DetailBuilder.GetDetailFont())
					.Text(this, &FFontFaceDetailsCustomization::GetFontDisplayName)
					.ToolTipText(this, &FFontFaceDetailsCustomization::GetFontDisplayToolTip)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("FontFilePathPickerToolTip", "Choose a font file from this computer"))
					.OnClicked(this, &FFontFaceDetailsCustomization::OnBrowseFontPath)
					.ContentPadding(2.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FEditorFontGlyphs::Folder_Open)
					]
				]
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							.ToolTipText(LOCTEXT("FontFileExportFontToolTip", "Export the font file to this computer"))
							.OnClicked(this, &FFontFaceDetailsCustomization::OnExportFont)
							.ContentPadding(2.0f)
							.ForegroundColor(FSlateColor::UseForeground())
							.IsFocusable(false)
							[
								SNew(STextBlock)
									.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
									.Text(FEditorFontGlyphs::File)
							]
					]
			];
	}

	// Distance field properties
	if (IsSlateSdfTextFeatureEnabled())
	{
		bool bShowDistanceFieldGroups = true;
		if (ObjectsBeingEdited.Num() == 1)
		{
			const UFontFace* FontFace = Cast<UFontFace>(ObjectsBeingEdited[0].Get());
			if (FontFace && !FontFace->bEnableDistanceFieldRendering)
			{
				bShowDistanceFieldGroups = false;
			}
		}
		if (bShowDistanceFieldGroups)
		{
			IDetailCategoryBuilder& DistanceFieldModeCategory = DetailBuilder.EditCategory("DistanceFieldMode");
			// Make sure that bEnableDistanceFieldRendering is listed first
			DistanceFieldModeCategory.AddProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, bEnableDistanceFieldRendering)));
			// Single-channel distance field resolution group
			IDetailGroup& SingleChannelResolutionGroup = DistanceFieldModeCategory.AddGroup(FName(TEXT("SingleChannelDistanceFieldResolution")), LOCTEXT("FontFaceSingleChannelDistanceFieldResolutionGroup", "Single-Channel Distance Field Resolution (px/em)"), false, true);
			SingleChannelResolutionGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MinDistanceFieldPpem)));
			SingleChannelResolutionGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MidDistanceFieldPpem)));
			SingleChannelResolutionGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MaxDistanceFieldPpem)));
			// Multi-channel distance field resolution group
			IDetailGroup& MultiChannelResolutionGroup = DistanceFieldModeCategory.AddGroup(FName(TEXT("MultiChannelDistanceFieldResolution")), LOCTEXT("FontFaceMultiChannelDistanceFieldResolutionGroup", "Multi-Channel Distance Field Resolution (px/em)"), false, true);
			MultiChannelResolutionGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MinMultiDistanceFieldPpem)));
			MultiChannelResolutionGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MidMultiDistanceFieldPpem)));
			MultiChannelResolutionGroup.AddPropertyRow(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MaxMultiDistanceFieldPpem)));
		}
	}
	else
	{
		// Hide distance field settings if disabled by CVar
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, bEnableDistanceFieldRendering))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MinDistanceFieldPpem))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MidDistanceFieldPpem))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MaxDistanceFieldPpem))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MinMultiDistanceFieldPpem))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MidMultiDistanceFieldPpem))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, MaxMultiDistanceFieldPpem))->MarkHiddenByCustomization();
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFontFace, PlatformRasterizationModeOverrides))->MarkHiddenByCustomization();
	}
}

FText FFontFaceDetailsCustomization::GetFontDisplayName() const
{
	if (ObjectsBeingEdited.Num() == 1)
	{
		const UFontFace* FontFace = Cast<UFontFace>(ObjectsBeingEdited[0].Get());
		if (FontFace && !FontFace->SourceFilename.IsEmpty())
		{
			return FText::FromString(FPaths::GetCleanFilename(FontFace->SourceFilename));
		}
	}

	return FText::GetEmpty();
}

FText FFontFaceDetailsCustomization::GetFontDisplayToolTip() const
{
	if (ObjectsBeingEdited.Num() == 1)
	{
		const UFontFace* FontFace = Cast<UFontFace>(ObjectsBeingEdited[0].Get());
		if (FontFace && !FontFace->SourceFilename.IsEmpty())
		{
			return FText::FromString(FontFace->SourceFilename);
		}
	}

	return FText::GetEmpty();
}

FReply FFontFaceDetailsCustomization::OnBrowseFontPath()
{
	IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform)
	{
		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);

		TArray<FString> OutFiles;
		if (DesktopPlatform->OpenFileDialog(
			nullptr,
			LOCTEXT("FontPickerTitle", "Choose a font file...").ToString(),
			DefaultPath,
			TEXT(""),
			Private::FileTypes,
			EFileDialogFlags::None,
			OutFiles
			))
		{
			OnFontPathPicked(OutFiles[0]);
		}
	}

	return FReply::Handled();
}

FReply FFontFaceDetailsCustomization::OnExportFont()
{
	IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);

		for (const TWeakObjectPtr<UObject>& ObjectBeingEdited : ObjectsBeingEdited)
		{
			UFontFace* FontFace = Cast<UFontFace>(ObjectBeingEdited.Get());
			if (FontFace)
			{
				const FString CurrentFilename = FPaths::GetBaseFilename(FontFace->SourceFilename);
				const FText Title = FText::Format(LOCTEXT("FontFace_ExportFontFileDialogTitle", "Export font '{0}'..."), FText::FromString(CurrentFilename));

				TArray<FString> OutFilenames;
				DesktopPlatform->SaveFileDialog(
					ParentWindowWindowHandle,
					Title.ToString(),
					(FontFace->SourceFilename.IsEmpty()) ? TEXT("") : FPaths::GetPath(FontFace->SourceFilename),
					(CurrentFilename.IsEmpty()) ? TEXT("") : CurrentFilename + TEXT(".ttf"),
					Private::FileTypes,
					EFileDialogFlags::None,
					OutFilenames
				);

				if (!OutFilenames.IsEmpty())
				{
					FFileHelper::SaveArrayToFile(FontFace->FontFaceData.Get().GetData(), *OutFilenames[0]);
				}
			}
		}
	}

	return FReply::Handled();
}

void FFontFaceDetailsCustomization::OnFontPathPicked(const FString& InNewFontFilename)
{
	TArray<uint8> FontData;
	FFileHelper::LoadFileToArray(FontData, *InNewFontFilename);

	const FScopedTransaction Transaction(LOCTEXT("SetFontFile", "Set Font File"));

	for (const TWeakObjectPtr<UObject>& ObjectBeingEdited : ObjectsBeingEdited)
	{
		UFontFace* FontFace = Cast<UFontFace>(ObjectBeingEdited.Get());
		if (FontFace)
		{
			FontFace->Modify();
			FontFace->SourceFilename = InNewFontFilename;
			FontFace->FontFaceData = FFontFaceData::MakeFontFaceData(MoveTemp(FontData)); // Make a new instance as the existing one may be being used by the font cache
			FontFace->CacheSubFaces();
			FontFace->PostEditChange();
		}
	}

	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InNewFontFilename));

	FSlateApplication::Get().GetRenderer()->FlushFontCache(TEXT("FFontFaceDetailsCustomization::OnFontPathPicked"));
}

#undef LOCTEXT_NAMESPACE
