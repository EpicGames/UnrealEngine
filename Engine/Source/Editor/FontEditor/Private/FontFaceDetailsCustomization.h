// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailLayoutBuilder;
class UObject;

/**
 *  Customize the font face asset to allow you to pick a file and store the result in the asset
 */
class FFontFaceDetailsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FFontFaceDetailsCustomization());
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	/** Get the leaf name of the font to show in the UI. */
	FText GetFontDisplayName() const;

	/** Get the full name of the font to show in the tooltip. */
	FText GetFontDisplayToolTip() const;

	/** Called in response to the user wanting to pick a new font file. */
	FReply OnBrowseFontPath();

	/** Called in response to the user wanting to export the font to a file. */
	FReply OnExportFont();

	/** Called in response to a new font file being picked. */
	void OnFontPathPicked(const FString& InNewFontFilename);

	/** Array of objects being edited by this customization. */
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingEdited;
};
