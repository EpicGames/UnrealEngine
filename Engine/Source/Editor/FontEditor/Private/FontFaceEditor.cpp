// Copyright Epic Games, Inc. All Rights Reserved.

#include "FontFaceEditor.h"

#include "Containers/Array.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorReimportHandler.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "Engine/UserInterfaceSettings.h"
#include "FontEditorModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/AppStyle.h"
#include "Subsystems/ImportSubsystem.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/ObjectPtr.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FontFaceEditor"

DEFINE_LOG_CATEGORY_STATIC(LogFontFaceEditor, Log, All);

const FName FFontFaceEditor::PreviewTabId( TEXT( "FontFaceEditor_FontFacePreview" ) );
const FName FFontFaceEditor::PropertiesTabId( TEXT( "FontFaceEditor_FontFaceProperties" ) );

void FFontFaceEditor::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_FontFaceEditor", "Font Face Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner( PreviewTabId,		FOnSpawnTab::CreateSP(this, &FFontFaceEditor::SpawnTab_Preview) )
		.SetDisplayName( LOCTEXT("PreviewTab", "Preview") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "FontEditor.Tabs.Preview"));

	InTabManager->RegisterTabSpawner( PropertiesTabId,	FOnSpawnTab::CreateSP(this, &FFontFaceEditor::SpawnTab_Properties) )
		.SetDisplayName( LOCTEXT("PropertiesTabId", "Details") )
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FFontFaceEditor::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner( PreviewTabId );	
	InTabManager->UnregisterTabSpawner( PropertiesTabId );
}

FFontFaceEditor::FFontFaceEditor()
	: FontFace(nullptr)
{
	PreviewRowVisibility[(int32)EPreviewRow::Reference] = true;
	PreviewRowVisibility[(int32)EPreviewRow::ApproximateSdfLow] = false;
	PreviewRowVisibility[(int32)EPreviewRow::ApproximateSdfMedium] = false;
	PreviewRowVisibility[(int32)EPreviewRow::ApproximateSdfHigh] = false;
	PreviewRowVisibility[(int32)EPreviewRow::SdfLow] = true;
	PreviewRowVisibility[(int32)EPreviewRow::SdfMedium] = true;
	PreviewRowVisibility[(int32)EPreviewRow::SdfHigh] = true;
	PreviewRowVisibility[(int32)EPreviewRow::MsdfLow] = true;
	PreviewRowVisibility[(int32)EPreviewRow::MsdfMedium] = true;
	PreviewRowVisibility[(int32)EPreviewRow::MsdfHigh] = true;
}

FFontFaceEditor::~FFontFaceEditor()
{
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

	if (UEditorEngine* Editor = Cast<UEditorEngine>(GEngine))
	{
		Editor->UnregisterForUndo(this);
		Editor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.RemoveAll(this);
	}
}

void FFontFaceEditor::InitFontFaceEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit)
{
	FReimportManager::Instance()->OnPostReimport().AddRaw(this, &FFontFaceEditor::OnPostReimport);

	// Register to be notified when an object is reimported.
	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetReimport.AddSP(this, &FFontFaceEditor::OnObjectReimported);

	FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &FFontFaceEditor::OnObjectPropertyChanged);

	FontFace = CastChecked<UFontFace>(ObjectToEdit);

	// Support undo/redo
	FontFace->SetFlags(RF_Transactional);
	
	if (UEditorEngine* Editor = Cast<UEditorEngine>(GEngine))
	{
		Editor->RegisterForUndo(this);
	}

	CreateInternalWidgets();

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_FontFaceEditor_Layout_v1")
	->AddArea
	(
		FTabManager::NewPrimaryArea() ->SetOrientation( Orient_Vertical )
		->Split
		(
			FTabManager::NewSplitter() ->SetOrientation(Orient_Vertical) ->SetSizeCoefficient(0.65f)
			->Split
			(
				FTabManager::NewStack() ->SetSizeCoefficient(0.85f)
				->AddTab( PropertiesTabId, ETabState::OpenedTab )
			)
			->Split
			(
				FTabManager::NewStack() ->SetSizeCoefficient(0.15f)
				->AddTab( PreviewTabId, ETabState::OpenedTab )
			)
		)
	);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, FontEditorAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectToEdit);

	IFontEditorModule* FontEditorModule = &FModuleManager::LoadModuleChecked<IFontEditorModule>("FontEditor");
	AddMenuExtender(FontEditorModule->GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));
}

UFontFace* FFontFaceEditor::GetFontFace() const
{
	return FontFace;
}

FName FFontFaceEditor::GetToolkitFName() const
{
	return FName("FontFaceEditor");
}

FText FFontFaceEditor::GetBaseToolkitName() const
{
	return LOCTEXT( "AppLabel", "Font Face Editor" );
}

FString FFontFaceEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Font Face ").ToString();
}

FLinearColor FFontFaceEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f);
}

TSharedRef<SDockTab> FFontFaceEditor::SpawnTab_Preview( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == PreviewTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("FontFacePreviewTitle", "Preview"))
		[
			FontFacePreview.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

TSharedRef<SDockTab> FFontFaceEditor::SpawnTab_Properties( const FSpawnTabArgs& Args )
{
	check( Args.GetTabId().TabType == PropertiesTabId );

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("FontFacePropertiesTitle", "Details"))
		[
			FontFaceProperties.ToSharedRef()
		];

	AddToSpawnedToolPanels( Args.GetTabId().TabType, SpawnedTab );

	return SpawnedTab;
}

void FFontFaceEditor::AddToSpawnedToolPanels( const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab )
{
	TWeakPtr<SDockTab>* TabSpot = SpawnedToolPanels.Find(TabIdentifier);
	if (!TabSpot)
	{
		SpawnedToolPanels.Add(TabIdentifier, SpawnedTab);
	}
	else
	{
		check(!TabSpot->IsValid());
		*TabSpot = SpawnedTab;
	}
}

void FFontFaceEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(FontFace);
	Collector.AddReferencedObjects(PreviewFonts);
	Collector.AddReferencedObjects(PreviewFaces);
}

void FFontFaceEditor::OnPreviewTextChanged(const FText& Text)
{
	for (TSharedPtr<STextBlock> &PreviewTextBlock : PreviewTextBlocks[1])
	{
		PreviewTextBlock->SetText(Text);
	}
}

TOptional<int32> FFontFaceEditor::GetPreviewFontSize() const
{
	return PreviewFontSize;
}

void FFontFaceEditor::OnPreviewFontSizeChanged(int32 InNewValue, ETextCommit::Type CommitType)
{
	PreviewFontSize = InNewValue;
	ApplyPreviewFontSize();
}

void FFontFaceEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{
	static const FName EnableDistanceFieldRenderingPropertyName = GET_MEMBER_NAME_CHECKED(UFontFace, bEnableDistanceFieldRendering);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == EnableDistanceFieldRenderingPropertyName)
	{
		// Show / hide distance field related properties
		FontFaceProperties->ForceRefresh();
	}

	RefreshPreview();
}

void FFontFaceEditor::CreateInternalWidgets()
{
	const EVerticalAlignment PreviewVAlign = VAlign_Center;
	const FText DefaultPreviewText = LOCTEXT("DefaultPreviewText", "The quick brown fox jumps over the lazy dog");

	FMenuBuilder PreviewRowVisibilitySelection(false, nullptr);

	auto AddPreviewVisibilityItem = [this, &PreviewRowVisibilitySelection](EPreviewRow Row, const TAttribute<FText>& InLabel, const TAttribute<FText>& InToolTip)
	{
		PreviewRowVisibilitySelection.AddMenuEntry(
			InLabel,
			InToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FFontFaceEditor::ChangePreviewRowVisibility, Row),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FFontFaceEditor::GetPreviewRowVisibility, Row)
			),
			NAME_None,
			EUserInterfaceActionType::Check
		);
	};

	AddPreviewVisibilityItem(
		EPreviewRow::Reference,
		LOCTEXT("FontFaceReferencePreviewVisibility", "Reference"),
		LOCTEXT("FontFaceReferencePreviewVisibilityTooltip", "Displays the Reference render of the preview text")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::ApproximateSdfLow,
		LOCTEXT("FontFaceApproximateSdfLowPreviewVisibility", "Approximate SDF Low Quality"),
		LOCTEXT("FontFaceApproximateSdfLowPreviewVisibilityTooltip", "Displays the preview text render of the fast approximation of the Low quality single-channel signed distance field")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::ApproximateSdfMedium,
		LOCTEXT("FontFaceApproximateSdfMediumPreviewVisibility", "Approximate SDF Medium Quality"),
		LOCTEXT("FontFaceApproximateSdfMediumPreviewVisibilityTooltip", "Displays the preview text render of the fast approximation of the Medium quality single-channel signed distance field")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::ApproximateSdfHigh,
		LOCTEXT("FontFaceApproximateSdfHighPreviewVisibility", "Approximate SDF High Quality"),
		LOCTEXT("FontFaceApproximateSdfHighPreviewVisibilityTooltip", "Displays the preview text render of the fast approximation of the High quality single-channel signed distance field")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::SdfLow,
		LOCTEXT("FontFaceSdfLowPreviewVisibility", "SDF Low Quality"),
		LOCTEXT("FontFaceSdfLowPreviewVisibilityTooltip", "Displays the Low quality single-channel signed distance field render of the preview text")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::SdfMedium,
		LOCTEXT("FontFaceSdfMediumPreviewVisibility", "SDF Medium Quality"),
		LOCTEXT("FontFaceSdfMediumPreviewVisibilityTooltip", "Displays the Medium quality single-channel signed distance field render of the preview text")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::SdfHigh,
		LOCTEXT("FontFaceSdfHighPreviewVisibility", "SDF High Quality"),
		LOCTEXT("FontFaceSdfHighPreviewVisibilityTooltip", "Displays the High quality single-channel signed distance field render of the preview text")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::MsdfLow,
		LOCTEXT("FontFaceMsdfLowPreviewVisibility", "MSDF Low Quality"),
		LOCTEXT("FontFaceMsdfLowPreviewVisibilityTooltip", "Displays the Low quality multi-channel signed distance field render of the preview text")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::MsdfMedium,
		LOCTEXT("FontFaceMsdfMediumPreviewVisibility", "MSDF Medium Quality"),
		LOCTEXT("FontFaceMsdfMediumPreviewVisibilityTooltip", "Displays the Medium quality multi-channel signed distance field render of the preview text")
	);
	AddPreviewVisibilityItem(
		EPreviewRow::MsdfHigh,
		LOCTEXT("FontFaceMsdfHighPreviewVisibility", "MSDF High Quality"),
		LOCTEXT("FontFaceMsdfHighPreviewVisibilityTooltip", "Displays the High quality multi-channel signed distance field render of the preview text")
	);

	FontFacePreview =
	SNew(SVerticalBox)
	+SVerticalBox::Slot()
	.FillHeight(1.0f)
	.Padding(0.0f, 0.0f, 0.0f, 4.0f)
	[
		SNew(SScrollBox)
		+SScrollBox::Slot()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SScrollBox)
				.Orientation(EOrientation::Orient_Horizontal)
				.ConsumeMouseWheel(EConsumeMouseWheel::Never)
				+SScrollBox::Slot()
				[
					SAssignNew(PreviewTextGridPanel, SGridPanel)
					+SGridPanel::Slot(0, (int32)EPreviewRow::Reference)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::Reference], STextBlock)
						.Text(LOCTEXT("FontFaceReferencePreviewLabel", "Reference: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::Reference)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::Reference], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::ApproximateSdfLow)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::ApproximateSdfLow], STextBlock)
						.Text(LOCTEXT("FontFaceApproximateSdfLowPreviewLabel", "ASDF Low: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::ApproximateSdfLow)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::ApproximateSdfLow], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::ApproximateSdfMedium)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::ApproximateSdfMedium], STextBlock)
						.Text(LOCTEXT("FontFaceApproximateSdfMediumPreviewLabel", "ASDF Medium: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::ApproximateSdfMedium)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::ApproximateSdfMedium], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::ApproximateSdfHigh)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::ApproximateSdfHigh], STextBlock)
						.Text(LOCTEXT("FontFaceApproximateSdfHighPreviewLabel", "ASDF High: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::ApproximateSdfHigh)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::ApproximateSdfHigh], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::SdfLow)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::SdfLow], STextBlock)
						.Text(LOCTEXT("FontFaceSdfLowPreviewLabel", "SDF Low: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::SdfLow)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::SdfLow], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::SdfMedium)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::SdfMedium], STextBlock)
						.Text(LOCTEXT("FontFaceSdfMediumPreviewLabel", "SDF Medium: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::SdfMedium)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::SdfMedium], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::SdfHigh)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::SdfHigh], STextBlock)
						.Text(LOCTEXT("FontFaceSdfHighPreviewLabel", "SDF High: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::SdfHigh)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::SdfHigh], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::MsdfLow)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::MsdfLow], STextBlock)
						.Text(LOCTEXT("FontFaceMsdfLowPreviewLabel", "MSDF Low: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::MsdfLow)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::MsdfLow], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::MsdfMedium)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::MsdfMedium], STextBlock)
						.Text(LOCTEXT("FontFaceMsdfMediumPreviewLabel", "MSDF Medium: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::MsdfMedium)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::MsdfMedium], STextBlock)
						.Text(DefaultPreviewText)
					]
					+SGridPanel::Slot(0, (int32)EPreviewRow::MsdfHigh)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[0][(int32)EPreviewRow::MsdfHigh], STextBlock)
						.Text(LOCTEXT("FontFaceMsdfHighPreviewLabel", "MSDF High: "))
					]
					+SGridPanel::Slot(1, (int32)EPreviewRow::MsdfHigh)
					.VAlign(PreviewVAlign)
					[
						SAssignNew(PreviewTextBlocks[1][(int32)EPreviewRow::MsdfHigh], STextBlock)
						.Text(DefaultPreviewText)
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(PreviewNoteTextBlock, STextBlock)
				.Text(LOCTEXT("FontFaceDistanceFieldProjectSettingNote", "Note: You must also enable Distance Field Font Rasterization in Project Settings / Engine / User Interface."))
				.Visibility(EVisibility::Collapsed)
			]
		]
	]
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		[
			SAssignNew(FontFacePreviewText, SEditableTextBox)
			.Text(DefaultPreviewText)
			.SelectAllTextWhenFocused(true)
			.OnTextChanged(this, &FFontFaceEditor::OnPreviewTextChanged)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SNumericEntryBox<int32>)
			.Value(this, &FFontFaceEditor::GetPreviewFontSize)
			.MinValue(4)
			.MaxValue(256)
			.OnValueCommitted(this, &FFontFaceEditor::OnPreviewFontSizeChanged)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(PreviewVisibilityButton, SComboButton)
			.HasDownArrow(false)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("FontFacePreviewVisibilityTooltip", "Selects which render modes to preview (requires Distance Field Rendering enabled)"))
			.MenuContent()
			[
				PreviewRowVisibilitySelection.MakeWidget()
			]
			.ButtonContent()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Level.VisibleIcon16x"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];

	UpdatePreviewFonts();
	UpdatePreviewVisibility();
	ApplyPreviewFontSize();

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.NotifyHook = this;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FontFaceProperties = PropertyModule.CreateDetailView(Args);

	FontFaceProperties->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateRaw(this, &FFontFaceEditor::GetIsPropertyVisible));
	FontFaceProperties->SetObject( FontFace );
}

void FFontFaceEditor::OnPostReimport(UObject* InObject, bool bSuccess)
{
	if (InObject == FontFace && bSuccess)
	{
		RefreshPreview();
	}
}

void FFontFaceEditor::OnObjectPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InObject == FontFace)
	{
		// Force all texts using a font to be refreshed.
		FSlateApplicationBase::Get().InvalidateAllWidgets(false);
		GSlateLayoutGeneration++;
		RefreshPreview();
	}
}

void FFontFaceEditor::OnObjectReimported(UObject* InObject)
{
	// Make sure we are using the object that is being reimported, otherwise a lot of needless work could occur.
	if (InObject == FontFace)
	{
		FontFace = Cast<UFontFace>(InObject);

		TArray< UObject* > ObjectList;
		ObjectList.Add(InObject);
		FontFaceProperties->SetObjects(ObjectList);
	}
}

bool FFontFaceEditor::GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	static const FName CategoryFName = "Category";
	const FString& CategoryValue = PropertyAndParent.Property.GetMetaData(CategoryFName);
	return CategoryValue != TEXT("DistanceFieldMode") || IsSlateSdfTextFeatureEnabled();
}

bool FFontFaceEditor::ShouldPromptForNewFilesOnReload(const UObject& EditingObject) const
{
	return false;
}

void FFontFaceEditor::RefreshPreview()
{
	UpdatePreviewFonts();
	UpdatePreviewVisibility();
}

void FFontFaceEditor::ClonePreviewFontFace(TObjectPtr<UFontFace>& TargetFontFace, EFontRasterizationMode RasterizationMode, int32 DistanceFieldPpem) const
{
	TargetFontFace = DuplicateObject<UFontFace>(FontFace, GetTransientPackage());
	TargetFontFace->MinDistanceFieldPpem = DistanceFieldPpem;
	TargetFontFace->MidDistanceFieldPpem = DistanceFieldPpem;
	TargetFontFace->MaxDistanceFieldPpem = DistanceFieldPpem;
	TargetFontFace->MinMultiDistanceFieldPpem = DistanceFieldPpem;
	TargetFontFace->MidMultiDistanceFieldPpem = DistanceFieldPpem;
	TargetFontFace->MaxMultiDistanceFieldPpem = DistanceFieldPpem;
	TargetFontFace->PlatformRasterizationModeOverrides = FFontFacePlatformRasterizationOverrides();
	TargetFontFace->PlatformRasterizationModeOverrides->MsdfOverride = RasterizationMode;
	TargetFontFace->PlatformRasterizationModeOverrides->SdfOverride = RasterizationMode;
	TargetFontFace->PlatformRasterizationModeOverrides->SdfApproximationOverride = RasterizationMode;
	TargetFontFace->PostEditChange();
}

void FFontFaceEditor::MakePreviewFont(TObjectPtr<UObject>& TargetObject, UFontFace* Face) const
{
	if (!TargetObject)
	{
		TargetObject = NewObject<UFont>();
	}
	if (UFont* TargetFont = CastChecked<UFont>(TargetObject))
	{
		FCompositeFont& MutableInternalCompositeFont = TargetFont->GetMutableInternalCompositeFont();
		if (MutableInternalCompositeFont.DefaultTypeface.Fonts.IsEmpty())
		{
			FTypefaceEntry FontTypeface;
			FontTypeface.Name = TEXT("Regular");
			FontTypeface.Font = FFontData(Face);
			TargetFont->FontCacheType = EFontCacheType::Runtime;
			MutableInternalCompositeFont.DefaultTypeface.Fonts.Add(MoveTemp(FontTypeface));
		}
		else
		{
			MutableInternalCompositeFont.DefaultTypeface.Fonts[0].Font = FFontData(Face);
		}
		TargetFont->PostEditChange();
	}
}

bool FFontFaceEditor::IsFontFaceDistanceFieldEnabled() const
{
	return FontFace->bEnableDistanceFieldRendering &&
	       GetDefault<UUserInterfaceSettings>()->bEnableDistanceFieldFontRasterization &&
	       IsSlateSdfTextFeatureEnabled();
}

void FFontFaceEditor::UpdatePreviewFonts()
{
	if (!FontFace)
	{
		return;
	}
	if (IsFontFaceDistanceFieldEnabled())
	{
		// This ensures that font geometry is preprocessed before cloning the face, otherwise it would be needlessly redone for each copy.
		FontFace->CacheSubFaces();
		PreviewFaces.SetNum((int32)EPreviewRow::Count, EAllowShrinking::No);
		PreviewFonts.SetNum((int32)EPreviewRow::Count, EAllowShrinking::No);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::Reference], EFontRasterizationMode::Bitmap);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::ApproximateSdfLow], EFontRasterizationMode::SdfApproximation, FontFace->MinDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::ApproximateSdfMedium], EFontRasterizationMode::SdfApproximation, FontFace->MidDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::ApproximateSdfHigh], EFontRasterizationMode::SdfApproximation, FontFace->MaxDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::SdfLow], EFontRasterizationMode::Sdf, FontFace->MinDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::SdfMedium], EFontRasterizationMode::Sdf, FontFace->MidDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::SdfHigh], EFontRasterizationMode::Sdf, FontFace->MaxDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::MsdfLow], EFontRasterizationMode::Msdf, FontFace->MinMultiDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::MsdfMedium], EFontRasterizationMode::Msdf, FontFace->MidMultiDistanceFieldPpem);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::MsdfHigh], EFontRasterizationMode::Msdf, FontFace->MaxMultiDistanceFieldPpem);
		for (int32 Index = 0; Index < (int32)EPreviewRow::Count; ++Index)
		{
			MakePreviewFont(PreviewFonts[Index], PreviewFaces[Index]);
		}
	}
	else
	{
		static_assert((int32)EPreviewRow::Reference == 0, "Reference preview font needs to be the only array element (i.e. position 0)");
		PreviewFaces.SetNum(1, EAllowShrinking::No);
		PreviewFonts.SetNum((int32)EPreviewRow::Count, EAllowShrinking::No);
		ClonePreviewFontFace(PreviewFaces[(int32)EPreviewRow::Reference], EFontRasterizationMode::Bitmap);
		for (TObjectPtr<UObject>& PreviewFont : PreviewFonts)
		{
			MakePreviewFont(PreviewFont, PreviewFaces[0]);
		}
	}
}

void FFontFaceEditor::UpdatePreviewVisibility()
{
	if (FontFace)
	{
		const bool bSecondaryRowsVisibility = IsFontFaceDistanceFieldEnabled();
		PreviewTextBlocks[0][(int32)EPreviewRow::Reference]->SetVisibility(bSecondaryRowsVisibility && PreviewRowVisibility[(int32)EPreviewRow::Reference] ? EVisibility::Visible : EVisibility::Collapsed);
		PreviewTextBlocks[1][(int32)EPreviewRow::Reference]->SetVisibility(!bSecondaryRowsVisibility || PreviewRowVisibility[(int32)EPreviewRow::Reference] ? EVisibility::Visible : EVisibility::Collapsed);
		static_assert((int32)EPreviewRow::Reference == 0, "Visibility not set for rows lower than EPreviewRow::Reference");
		for (int32 PreviewRow = (int32)EPreviewRow::Reference + 1; PreviewRow < (int32)EPreviewRow::Count; ++PreviewRow)
		{
			const EVisibility RowVisibility = bSecondaryRowsVisibility && PreviewRowVisibility[PreviewRow] ? EVisibility::Visible : EVisibility::Collapsed;
			PreviewTextBlocks[0][PreviewRow]->SetVisibility(RowVisibility);
			PreviewTextBlocks[1][PreviewRow]->SetVisibility(RowVisibility);
		}
		PreviewNoteTextBlock->SetVisibility(
			FontFace->bEnableDistanceFieldRendering &&
			IsSlateSdfTextFeatureEnabled() &&
			!GetDefault<UUserInterfaceSettings>()->bEnableDistanceFieldFontRasterization ?
			EVisibility::Visible : EVisibility::Collapsed
		);
		PreviewVisibilityButton->SetEnabled(bSecondaryRowsVisibility);
	}
	else
	{
		for (int32 PreviewRow = 0; PreviewRow < (int32)EPreviewRow::Count; ++PreviewRow)
		{
			PreviewTextBlocks[0][PreviewRow]->SetVisibility(EVisibility::Collapsed);
			PreviewTextBlocks[1][PreviewRow]->SetVisibility(EVisibility::Collapsed);
		}
		PreviewVisibilityButton->SetEnabled(false);
	}
}

void FFontFaceEditor::ApplyPreviewFontSize()
{
	constexpr const int32 ColumnIndex = 1;
	for (int32 RowIndex = 0; RowIndex < UE_ARRAY_COUNT(PreviewTextBlocks[ColumnIndex]) && RowIndex < PreviewFonts.Num(); ++RowIndex)
	{
		const TSharedPtr<STextBlock>& PreviewTextBlock = PreviewTextBlocks[ColumnIndex][RowIndex];
		PreviewTextBlock->SetFont(FSlateFontInfo(PreviewFonts[RowIndex], PreviewFontSize));

		if (TPanelChildren<SGridPanel::FSlot>* PreviewTextGridPanelChildren = static_cast<TPanelChildren<SGridPanel::FSlot>*>(PreviewTextGridPanel->GetChildren()))
		{
			TPanelChildren<SGridPanel::FSlot>& Children = *PreviewTextGridPanelChildren;
			TSharedRef<STextBlock> TextBlock = PreviewTextBlock.ToSharedRef();
			for (int32 SlotIndex = 0; SlotIndex < Children.Num(); ++SlotIndex)
			{
				SGridPanel::FSlot& Slot = Children[SlotIndex];
				if (Slot.GetWidget() == TextBlock)
				{
					Slot.SetPadding(GetPreviewTextPadding());
					break;
				}
			}
		}

		PreviewTextGridPanel->Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void FFontFaceEditor::ChangePreviewRowVisibility(EPreviewRow Row)
{
	int32 RowIndex = (int32)Row;
	check(RowIndex >= 0 && RowIndex < (int32)EPreviewRow::Count);
	PreviewRowVisibility[RowIndex] = !PreviewRowVisibility[RowIndex];
	UpdatePreviewVisibility();
}

bool FFontFaceEditor::GetPreviewRowVisibility(EPreviewRow Row) const
{
	int32 RowIndex = (int32)Row;
	check(RowIndex >= 0 && RowIndex < (int32)EPreviewRow::Count);
	return PreviewRowVisibility[RowIndex];
}

float FFontFaceEditor::GetPreviewTextPadding() const
{
	return PreviewFontSize / 2.0f;
}

#undef LOCTEXT_NAMESPACE
