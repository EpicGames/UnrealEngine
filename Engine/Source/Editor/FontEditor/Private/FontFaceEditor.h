// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "IFontFaceEditor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "UObject/GCObject.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"

class FReferenceCollector;
class SComboButton;
class SCompositeFontEditor;
class SDockTab;
class SEditableTextBox;
class SGridPanel;
class SVerticalBox;
class UFontFace;
struct FPropertyChangedEvent;

/*-----------------------------------------------------------------------------
   FFontFaceEditor
-----------------------------------------------------------------------------*/

class FFontFaceEditor : public IFontFaceEditor, public FGCObject, public FNotifyHook, public FEditorUndoClient
{
public:
	FFontFaceEditor();

	virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;

	/** Destructor */
	virtual ~FFontFaceEditor();

	/** Edits the specified Font object */
	void InitFontFaceEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UObject* ObjectToEdit);

	/** IFontFaceEditor interface */
	virtual UFontFace* GetFontFace() const override;
	virtual void RefreshPreview() override;
	
	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FFontFaceEditor");
	}

	/** Called to determine if the user should be prompted for a new file if one is missing during an asset reload*/
	virtual bool ShouldPromptForNewFilesOnReload(const UObject& object) const override;

protected:
	/** Called when the preview text changes */
	void OnPreviewTextChanged(const FText& Text);

	/** Called to handle the "Preview Font Size" numeric entry box */
	TOptional<int32> GetPreviewFontSize() const;
	void OnPreviewFontSizeChanged(int32 InNewValue, ETextCommit::Type CommitType);

private:
	/** FNotifyHook interface */
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

	/** Creates all internal widgets for the tabs to point at */
	void CreateInternalWidgets();

	void OnPostReimport(UObject* InObject, bool bSuccess);
	void OnObjectPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);

	/** Spawns the preview tab */
	TSharedRef<SDockTab> SpawnTab_Preview(const FSpawnTabArgs& Args);

	/** Spawns the properties tab */
	TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);

	/** Caches the specified tab for later retrieval */
	void AddToSpawnedToolPanels(const FName& TabIdentifier, const TSharedRef<SDockTab>& SpawnedTab);

	/** Callback when an object is reimported, handles steps needed to keep the editor up-to-date. */
	void OnObjectReimported(UObject* InObject);

	/** Check to see if the given property should be visible in the details panel */
	bool GetIsPropertyVisible(const struct FPropertyAndParent& PropertyAndParent) const;

private:

	/** Preview text rows that can have their visibility toggled */
	enum class EPreviewRow : int32
	{
		Reference,
		ApproximateSdfLow,
		ApproximateSdfMedium,
		ApproximateSdfHigh,
		SdfLow,
		SdfMedium,
		SdfHigh,
		MsdfLow,
		MsdfMedium,
		MsdfHigh,
		Count
	};

	/** The font asset being inspected */
	TObjectPtr<UFontFace> FontFace;

	/** Virtual fonts for editor preview only */
	TArray<TObjectPtr<UObject>> PreviewFonts;

	/** Virtual font faces for editor preview only */
	TArray<TObjectPtr<UFontFace>> PreviewFaces;

	/** List of open tool panels; used to ensure only one exists at any one time */
	TMap< FName, TWeakPtr<SDockTab> > SpawnedToolPanels;

	/** Preview tab */
	TSharedPtr<SVerticalBox> FontFacePreview;

	/** Properties tab */
	TSharedPtr<class IDetailsView> FontFaceProperties;

	/** Preview text */
	TSharedPtr<SEditableTextBox> FontFacePreviewText;

	/** Preview visibility menu button */
	TSharedPtr<SComboButton> PreviewVisibilityButton;

	/** Preview widgets */
	TSharedPtr<STextBlock> PreviewTextBlocks[2][(int32)EPreviewRow::Count];

	/** Preview grid panel */
	TSharedPtr<SGridPanel> PreviewTextGridPanel;

	/** Preview note text */
	TSharedPtr<STextBlock> PreviewNoteTextBlock;

	/** Preview font size */
	int32 PreviewFontSize = 24;

	/** Preview row user visibility filter */
	bool PreviewRowVisibility[(int32)EPreviewRow::Count];

	/** The tab ids for the font editor */
	static const FName PreviewTabId;
	static const FName PropertiesTabId;

	void ClonePreviewFontFace(TObjectPtr<UFontFace>& TargetFontFace, EFontRasterizationMode RasterizationMode, int32 DistanceFieldPpem = 0) const;
	void MakePreviewFont(TObjectPtr<UObject>& TargetObject, UFontFace* Face) const;
	bool IsFontFaceDistanceFieldEnabled() const;

	void UpdatePreviewFonts();
	void UpdatePreviewVisibility();
	void ApplyPreviewFontSize();
	void ChangePreviewRowVisibility(EPreviewRow Row);
	bool GetPreviewRowVisibility(EPreviewRow Row) const;
	float GetPreviewTextPadding() const;
};
