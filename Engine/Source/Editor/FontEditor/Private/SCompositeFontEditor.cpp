// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCompositeFontEditor.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Containers/ArrayView.h"
#include "ContentBrowserModule.h"
#include "CoreTypes.h"
#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorDirectories.h"
#include "EditorFontGlyphs.h"
#include "Engine/Font.h"
#include "Engine/FontFace.h"
#include "Factories/FontFileImportFactory.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/FontCache.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Views/ITypedTableView.h"
#include "GenericPlatform/GenericWindow.h"
#include "IContentBrowserSingleton.h"
#include "IDesktopPlatform.h"
#include "IFontEditor.h"
#include "Internationalization/Internationalization.h"
#include "Layout/BasicLayoutWidgetSlot.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Range.h"
#include "Misc/AssertionMacros.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "Rendering/SlateRenderer.h"
#include "ScopedTransaction.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/ImportSubsystem.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "FontEditor"

/** Entry used to weakly reference a particular typeface entry in the SListView */
struct FTypefaceListViewEntry
{
	TAttribute<FTypeface*> Typeface;
	int32 TypefaceEntryIndex;
	bool bRenameRequested;

	FTypefaceListViewEntry()
		: Typeface()
		, TypefaceEntryIndex(INDEX_NONE)
		, bRenameRequested(false)
	{
	}

	FTypefaceListViewEntry(TAttribute<FTypeface*>& InTypeface, const int32 InTypefaceEntryIndex)
		: Typeface(InTypeface)
		, TypefaceEntryIndex(InTypefaceEntryIndex)
		, bRenameRequested(false)
	{
	}

	void Reset()
	{
		*this = FTypefaceListViewEntry();
	}

	FTypefaceEntry* GetTypefaceEntry() const
	{
		FTypeface* const TypefacePtr = Typeface.Get(nullptr);
		return (TypefacePtr && TypefaceEntryIndex < TypefacePtr->Fonts.Num()) ? &TypefacePtr->Fonts[TypefaceEntryIndex] : nullptr;
	}
};

/** Entry used to weakly reference a particular sub-typeface entry in the SListView */
struct FSubTypefaceListViewEntry
{
	FCompositeFont* CompositeFont;
	int32 SubTypefaceEntryIndex;
	bool bRenameRequested;

	FSubTypefaceListViewEntry()
		: CompositeFont(nullptr)
		, SubTypefaceEntryIndex(INDEX_NONE)
		, bRenameRequested(false)
	{
	}

	FSubTypefaceListViewEntry(FCompositeFont* const InCompositeFont, const int32 InSubTypefaceEntryIndex)
		: CompositeFont(InCompositeFont)
		, SubTypefaceEntryIndex(InSubTypefaceEntryIndex)
		, bRenameRequested(false)
	{
	}

	void Reset()
	{
		*this = FSubTypefaceListViewEntry();
	}

	FCompositeSubFont* GetSubTypefaceEntry() const
	{
		return (CompositeFont && SubTypefaceEntryIndex < CompositeFont->SubTypefaces.Num()) ? &CompositeFont->SubTypefaces[SubTypefaceEntryIndex] : nullptr;
	}
};

/** Entry used to weakly reference a particular character range entry in the STileView */
struct FCharacterRangeTileViewEntry
{
	FSubTypefaceListViewEntryPtr SubTypefaceEntry;
	int32 RangeEntryIndex;

	FCharacterRangeTileViewEntry()
		: SubTypefaceEntry()
		, RangeEntryIndex(INDEX_NONE)
	{
	}

	FCharacterRangeTileViewEntry(FSubTypefaceListViewEntryPtr InSubTypefaceEntry, const int32 InRangeEntryIndex)
		: SubTypefaceEntry(InSubTypefaceEntry)
		, RangeEntryIndex(InRangeEntryIndex)
	{
	}

	void Reset()
	{
		*this = FCharacterRangeTileViewEntry();
	}

	FInt32Range* GetRange() const
	{
		FCompositeSubFont* const SubTypefaceEntryPtr = (SubTypefaceEntry.IsValid()) ? SubTypefaceEntry->GetSubTypefaceEntry() : nullptr;
		return (SubTypefaceEntryPtr && RangeEntryIndex < SubTypefaceEntryPtr->CharacterRanges.Num()) ? &SubTypefaceEntryPtr->CharacterRanges[RangeEntryIndex] : nullptr;
	}
};

SCompositeFontEditor::~SCompositeFontEditor()
{
}

void SCompositeFontEditor::Construct(const FArguments& InArgs)
{
	FontEditorPtr = InArgs._FontEditor;

	// Flush the cached font on a culture change to make sure the preview updates correctly
	FInternationalization::Get().OnCultureChanged().AddSP(this, &SCompositeFontEditor::FlushCachedFont);

	ChildSlot
	[
		SNew(SScrollBox)

		+SScrollBox::Slot()
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(DefaultTypefaceEditor, STypefaceEditor)
				.CompositeFontEditor(this)
				.Typeface(this, &SCompositeFontEditor::GetDefaultTypeface)
				.TypefaceDisplayName(LOCTEXT("DefaultFontFamilyName", "Default Font Family"))
				.TypefaceDisplayNameToolTip(LOCTEXT("DefaultFontFamilyToolTip", "The font family that will be used as the primary font source, all sub-font families should match the size and style of this one"))
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(FallbackTypefaceEditor, STypefaceEditor)
				.CompositeFontEditor(this)
				.Typeface(this, &SCompositeFontEditor::GetFallbackTypeface)
				.TypefaceDisplayName(LOCTEXT("FallbackFontFamilyName", "Fallback Font Family"))
				.TypefaceDisplayNameToolTip(LOCTEXT("FallbackFontFamilyToolTip", "The font family that will be used as a catch-all fallback when neither the default font family or any sub-font families support a character"))
				.HeaderContent()
				[
					SNew(SBox)
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(4.0f, 0.0f))
						[
							SNew(SFontScalingFactorEditor)
							.CompositeFontEditor(this)
							.FallbackFont(this, &SCompositeFontEditor::GetFallbackFont)
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SFontOverrideSelector)
							.CompositeFontEditor(this)
							.TypefaceEditor_Lambda([this] { return FallbackTypefaceEditor.Get(); })
							.Typeface(this, &SCompositeFontEditor::GetFallbackTypeface)
							.ParentTypeface(this, &SCompositeFontEditor::GetConstDefaultTypeface)
						]

						// Hidden button using the same image as the sub-fonts to act as an alignment padding
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
						[
							SNew(SButton)
							.Visibility(EVisibility::Hidden)
							.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
							[
								SNew(SImage)
								.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Delete"))
							]
						]
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SAssignNew(SubTypefaceEntriesListView, SListView<FSubTypefaceListViewEntryPtr>)
				.ListItemsSource(&SubTypefaceEntries)
				.SelectionMode(ESelectionMode::None)
				.OnGenerateRow(this, &SCompositeFontEditor::MakeSubTypefaceEntryWidget)
			]
		]
	];

	UpdateSubTypefaceList();
}

void SCompositeFontEditor::Refresh()
{
	FlushCachedFont();

	DefaultTypefaceEditor->Refresh();
	FallbackTypefaceEditor->Refresh();
	UpdateSubTypefaceList();
}

void SCompositeFontEditor::FlushCachedFont()
{
	FCompositeFont* const CompositeFont = GetCompositeFont();
	if(CompositeFont)
	{
		CompositeFont->MakeDirty();
		FSlateApplication::Get().GetRenderer()->GetFontCache()->FlushCompositeFont(*CompositeFont);
	}

	TSharedPtr<IFontEditor> FontEditor = FontEditorPtr.Pin();
	if(FontEditor.IsValid())
	{
		FontEditor->RefreshPreview();
	}
}

UFont* SCompositeFontEditor::GetFontObject() const
{
	TSharedPtr<IFontEditor> FontEditor = FontEditorPtr.Pin();
	return (FontEditor.IsValid()) ? FontEditor->GetFont() : nullptr;
}

FCompositeFont* SCompositeFontEditor::GetCompositeFont() const
{
	UFont* const FontObject = GetFontObject();
	return (FontObject) ? &FontObject->GetMutableInternalCompositeFont() : nullptr;
}

FTypeface* SCompositeFontEditor::GetDefaultTypeface() const
{
	UFont* const FontObject = GetFontObject();
	return (FontObject) ? &FontObject->GetMutableInternalCompositeFont().DefaultTypeface : nullptr;
}

const FTypeface* SCompositeFontEditor::GetConstDefaultTypeface() const
{
	return GetDefaultTypeface();
}

FTypeface* SCompositeFontEditor::GetFallbackTypeface() const
{
	UFont* const FontObject = GetFontObject();
	return (FontObject) ? &FontObject->GetMutableInternalCompositeFont().FallbackTypeface.Typeface : nullptr;
}

const FTypeface* SCompositeFontEditor::GetConstFallbackTypeface() const
{
	return GetFallbackTypeface();
}

FCompositeFallbackFont* SCompositeFontEditor::GetFallbackFont() const
{
	UFont* const FontObject = GetFontObject();
	return (FontObject) ? &FontObject->GetMutableInternalCompositeFont().FallbackTypeface : nullptr;
}

void SCompositeFontEditor::UpdateSubTypefaceList()
{
	for(FSubTypefaceListViewEntryPtr& SubTypefaceListViewEntry : SubTypefaceEntries)
	{
		SubTypefaceListViewEntry->Reset();
	}

	FCompositeFont* const CompositeFontPtr = GetCompositeFont();
	if(CompositeFontPtr)
	{
		SubTypefaceEntries.Empty(CompositeFontPtr->SubTypefaces.Num());

		for(int32 SubTypefaceEntryIndex = 0; SubTypefaceEntryIndex < CompositeFontPtr->SubTypefaces.Num(); ++SubTypefaceEntryIndex)
		{
			SubTypefaceEntries.Add(MakeShareable(new FSubTypefaceListViewEntry(CompositeFontPtr, SubTypefaceEntryIndex)));
		}
	}

	// Add a dummy entry for the "Add" button slot
	SubTypefaceEntries.Add(MakeShareable(new FSubTypefaceListViewEntry()));

	SubTypefaceEntriesListView->RequestListRefresh();
}

TSharedRef<ITableRow> SCompositeFontEditor::MakeSubTypefaceEntryWidget(FSubTypefaceListViewEntryPtr InSubTypefaceEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SWidget> EntryWidget;

	if(InSubTypefaceEntry->SubTypefaceEntryIndex == INDEX_NONE)
	{
		// Dummy entry for the "Add" button
		EntryWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("AddSubFontFamilyTooltip", "Add a sub-font family to this composite font"))
			.OnClicked(this, &SCompositeFontEditor::OnAddSubFontFamily)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Add"))
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(LOCTEXT("AddSubFontFamily", "Add Sub-Font Family"))
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
				]
			]
		];
	}
	else
	{
		EntryWidget = SNew(SSubTypefaceEditor)
		.CompositeFontEditor(this)
		.SubTypeface(InSubTypefaceEntry)
		.ParentTypeface(this, &SCompositeFontEditor::GetConstDefaultTypeface)
		.OnDeleteSubFontFamily(this, &SCompositeFontEditor::OnDeleteSubFontFamily);
	}

	return
		SNew(STableRow<FSubTypefaceListViewEntryPtr>, OwnerTable)
		[
			EntryWidget.ToSharedRef()
		];
}

FReply SCompositeFontEditor::OnAddSubFontFamily()
{
	const FScopedTransaction Transaction(LOCTEXT("AddSubFontFamily", "Add Sub-Font Family"));
	GetFontObject()->Modify();

	FCompositeFont* const CompositeFontPtr = GetCompositeFont();
	if(CompositeFontPtr)
	{
		const int32 NewSubFontIndex = CompositeFontPtr->SubTypefaces.Add(FCompositeSubFont());
		UpdateSubTypefaceList();

		// Ask for the newly added entry to be renamed to draw attention to it
		check(SubTypefaceEntries.IsValidIndex(NewSubFontIndex));
		SubTypefaceEntries[NewSubFontIndex]->bRenameRequested = true;

		FlushCachedFont();
	}

	return FReply::Handled();
}

void SCompositeFontEditor::OnDeleteSubFontFamily(const FSubTypefaceListViewEntryPtr& SubTypefaceEntryToRemove)
{
	const FScopedTransaction Transaction(LOCTEXT("DeleteSubFontFamily", "Delete Sub-Font Family"));
	GetFontObject()->Modify();

	FCompositeFont* const CompositeFontPtr = GetCompositeFont();
	if(CompositeFontPtr)
	{
		CompositeFontPtr->SubTypefaces.RemoveAt(SubTypefaceEntryToRemove->SubTypefaceEntryIndex);
		UpdateSubTypefaceList();

		FlushCachedFont();
	}
}

STypefaceEditor::~STypefaceEditor()
{
}

void STypefaceEditor::Construct(const FArguments& InArgs)
{
	CompositeFontEditorPtr = InArgs._CompositeFontEditor;
	Typeface = InArgs._Typeface;

	ChildSlot
	.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
		.Padding(0.0f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(8.0f, 8.0f, 16.0f, 8.0f))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				[
					SAssignNew(NameEditableTextBox, SInlineEditableTextBlock)
					.Text(InArgs._TypefaceDisplayName)
					.ToolTipText(InArgs._TypefaceDisplayNameToolTip)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.OnTextCommitted(InArgs._OnDisplayNameCommitted)
					.IsReadOnly(!InArgs._OnDisplayNameCommitted.IsBound())
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					InArgs._HeaderContent.Widget
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				InArgs._BodyContent.Widget
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(8.0f, 0.0f, 8.0f, 0.0f))
			[
				SAssignNew(TypefaceEntriesTileView, STileView<FTypefaceListViewEntryPtr>)
				.ListItemsSource(&TypefaceEntries)
				.SelectionMode(ESelectionMode::None)
				.ItemWidth(180)
				.ItemHeight(120)
				.ItemAlignment(EListItemAlignment::LeftAligned)
				.OnGenerateTile(this, &STypefaceEditor::MakeTypefaceEntryWidget)
			]
		]
	];

	UpdateFontList();
}

void STypefaceEditor::Refresh()
{
	UpdateFontList();
}

void STypefaceEditor::RequestRename()
{
	NameEditableTextBox->EnterEditingMode();
}

void STypefaceEditor::UpdateFontList()
{
	FTypeface* const TypefacePtr = Typeface.Get(nullptr);

	for(FTypefaceListViewEntryPtr& TypefaceListViewEntry : TypefaceEntries)
	{
		TypefaceListViewEntry->Reset();
	}

	TypefaceEntries.Empty((TypefacePtr) ? TypefacePtr->Fonts.Num() : 0);

	if(TypefacePtr)
	{
		for(int32 TypefaceEntryIndex = 0; TypefaceEntryIndex < TypefacePtr->Fonts.Num(); ++TypefaceEntryIndex)
		{
			TypefaceEntries.Add(MakeShareable(new FTypefaceListViewEntry(Typeface, TypefaceEntryIndex)));
		}
	}

	// Add a dummy entry for the "Add" button slot
	TypefaceEntries.Add(MakeShareable(new FTypefaceListViewEntry()));

	TypefaceEntriesTileView->RequestListRefresh();
}

TSharedRef<ITableRow> STypefaceEditor::MakeTypefaceEntryWidget(FTypefaceListViewEntryPtr InTypefaceEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SWidget> EntryWidget;

	if(InTypefaceEntry->TypefaceEntryIndex == INDEX_NONE)
	{
		// Dummy entry for the "Add" button
		EntryWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("AddFontTooltip", "Add a new font to this font family"))
			.OnClicked(this, &STypefaceEditor::OnAddFont)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.0f)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Add"))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(LOCTEXT("AddFont", "Add Font"))
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Justification(ETextJustify::Center)
				]
			]
		];
	}
	else
	{
		EntryWidget = SNew(STypefaceEntryEditor)
		.CompositeFontEditor(CompositeFontEditorPtr)
		.TypefaceEntry(InTypefaceEntry)
		.OnDeleteFont(this, &STypefaceEditor::OnDeleteFont)
		.OnVerifyFontName(this, &STypefaceEditor::OnVerifyFontName);
	}

	return
		SNew(STableRow<FTypefaceListViewEntryPtr>, OwnerTable)
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 8.0f, 8.0f))
			[
				EntryWidget.ToSharedRef()
			]
		];
}

FReply STypefaceEditor::OnAddFont()
{
	FTypeface* const TypefacePtr = Typeface.Get(nullptr);

	if(TypefacePtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddFont", "Add Font"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		TSet<FName> ExistingFontNames;
		for(const FTypefaceEntry& TypefaceEntry : TypefacePtr->Fonts)
		{
			ExistingFontNames.Add(TypefaceEntry.Name);
		}

		// Get a valid default name for the font
		static const FName BaseFontName("Font");
		FName NewFontName = BaseFontName;
		while(ExistingFontNames.Contains(NewFontName))
		{
			NewFontName.SetNumber(NewFontName.GetNumber() + 1);
		}

		const int32 NewEntryIndex = TypefacePtr->Fonts.Add(FTypefaceEntry(NewFontName));
		UpdateFontList();

		// Ask for the newly added entry to be renamed to draw attention to it
		check(TypefaceEntries.IsValidIndex(NewEntryIndex));
		TypefaceEntries[NewEntryIndex]->bRenameRequested = true;

		CompositeFontEditorPtr->FlushCachedFont();
	}

	return FReply::Handled();
}

void STypefaceEditor::OnDeleteFont(const FTypefaceListViewEntryPtr& TypefaceEntryToRemove)
{
	FTypeface* const TypefacePtr = Typeface.Get(nullptr);

	if(TypefacePtr && TypefaceEntryToRemove.IsValid() && TypefaceEntryToRemove->GetTypefaceEntry())
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteFont", "Delete Font"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		TypefacePtr->Fonts.RemoveAt(TypefaceEntryToRemove->TypefaceEntryIndex);
		UpdateFontList();

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

bool STypefaceEditor::OnVerifyFontName(const FTypefaceListViewEntryPtr& TypefaceEntryBeingRenamed, const FName& NewName, FText& OutFailureReason) const
{
	FTypeface* const TypefacePtr = Typeface.Get(nullptr);
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntryBeingRenamed->GetTypefaceEntry();

	// Empty names are invalid
	if(NewName.IsNone())
	{
		OutFailureReason = LOCTEXT("Error_FontNameEmpty", "The font name cannot be empty or 'None'");
		return false;
	}

	// If we already have this name, it's valid
	if(TypefaceEntryPtr && TypefaceEntryPtr->Name == NewName)
	{
		return true;
	}

	// Duplicate names are invalid
	if(TypefacePtr)
	{
		const bool bNameExists = TypefacePtr->Fonts.ContainsByPredicate([&NewName](const FTypefaceEntry& TypefaceEntry) -> bool
		{
			return TypefaceEntry.Name == NewName;
		});

		if(bNameExists)
		{
			OutFailureReason = FText::Format(LOCTEXT("Error_DuplicateFontNameFmt", "A font with the name '{0}' already exists"), FText::FromName(NewName));
			return false;
		}
	}

	return true;
}

STypefaceEntryEditor::~STypefaceEntryEditor()
{
}

void STypefaceEntryEditor::Construct(const FArguments& InArgs)
{
	CompositeFontEditorPtr = InArgs._CompositeFontEditor;
	TypefaceEntry = InArgs._TypefaceEntry;
	OnDeleteFont = InArgs._OnDeleteFont;
	OnVerifyFontName = InArgs._OnVerifyFontName;

	CacheSubFaceData();

	TSharedPtr<FSubFaceInfo> InitiallySelectedSubFace;
	if (FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry())
	{
		const int32 SubFaceIndex = TypefaceEntryPtr->Font.GetSubFaceIndex();
		if (SubFacesData.IsValidIndex(SubFaceIndex))
		{
			InitiallySelectedSubFace = SubFacesData[SubFaceIndex];
		}
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(8.0f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			[
				SAssignNew(NameEditableTextBox, SInlineEditableTextBlock)
				.Text(this, &STypefaceEntryEditor::GetTypefaceEntryName)
				.ToolTipText(LOCTEXT("FontNameTooltip", "The name of this font within the font family (click to edit)"))
				.OnTextCommitted(this, &STypefaceEntryEditor::OnTypefaceEntryNameCommitted)
				.OnVerifyTextChanged(this, &STypefaceEntryEditor::OnTypefaceEntryChanged)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UFontFace::StaticClass())
					.ObjectPath(this, &STypefaceEntryEditor::GetFontFaceAssetPath)
					.OnObjectChanged(this, &STypefaceEntryEditor::OnFontFaceAssetChanged)
					.DisplayUseSelected(false)
					.DisplayBrowse(false)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("FontFilePathPickerToolTip", "Choose a font file from this computer"))
					.OnClicked(this, &STypefaceEntryEditor::OnBrowseTypefaceEntryFontPath)
					.ContentPadding(2.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(STextBlock)
						.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
						.Text(FEditorFontGlyphs::Folder_Open)
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			[
				SAssignNew(SubFacesCombo, SComboBox<TSharedPtr<FSubFaceInfo>>)
				.Visibility(this, &STypefaceEntryEditor::GetSubFaceVisibility)
				.OptionsSource(&SubFacesData)
				.InitiallySelectedItem(InitiallySelectedSubFace)
				.ContentPadding(FMargin(4.0, 2.0))
				.OnSelectionChanged(this, &STypefaceEntryEditor::OnSubFaceSelectionChanged)
				.OnGenerateWidget(this, &STypefaceEntryEditor::MakeSubFaceSelectionWidget)
				[
					SNew(STextBlock)
					.Text(this, &STypefaceEntryEditor::GetCurrentSubFaceSelectionDisplayName)
					.ToolTipText(this, &STypefaceEntryEditor::GetCurrentSubFaceSelectionDisplayName)
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 0.0f, 0.0f, 4.0f))
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("FontFaceUpgradeToolTip", "This font face has been upgraded from legacy data and needs to be split into its own asset before it can be edited."))
				.Visibility(this, &STypefaceEntryEditor::GetUpgradeDataVisibility)
				.OnClicked(this, &STypefaceEntryEditor::OnUpgradeDataClicked)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("FontFaceUpgradeBtn", "Upgrade Data"))
					]
				]
			]

			+SVerticalBox::Slot()
			[
				SNew(SSpacer)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("DeleteFontTooltip", "Remove this font from the font family"))
				.OnClicked(this, &STypefaceEntryEditor::OnDeleteFontClicked)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Delete"))
				]
			]
		]
	];
}

void STypefaceEntryEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (TypefaceEntry.IsValid() && TypefaceEntry->bRenameRequested)
	{
		TypefaceEntry->bRenameRequested = false;
		NameEditableTextBox->EnterEditingMode();
	}
}

FText STypefaceEntryEditor::GetTypefaceEntryName() const
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if(TypefaceEntryPtr)
	{
		return FText::FromName(TypefaceEntryPtr->Name);
	}

	return FText::GetEmpty();
}

void STypefaceEntryEditor::OnTypefaceEntryNameCommitted(const FText& InNewName, ETextCommit::Type InCommitType)
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if(TypefaceEntryPtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("RenameFont", "Rename Font"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		TypefaceEntryPtr->Name = *InNewName.ToString();

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

bool STypefaceEntryEditor::OnTypefaceEntryChanged(const FText& InNewName, FText& OutFailureReason) const
{
	return !OnVerifyFontName.IsBound() || OnVerifyFontName.Execute(TypefaceEntry, *InNewName.ToString(), OutFailureReason);
}

FString STypefaceEntryEditor::GetFontFaceAssetPath() const
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if(TypefaceEntryPtr)
	{
		const UFontFace* FontFaceAsset = Cast<const UFontFace>(TypefaceEntryPtr->Font.GetFontFaceAsset());

		// Don't show the path for font faces within the same package as the main font (these have been in-place upgraded and should be split into their own package)
		if (FontFaceAsset && FontFaceAsset->GetOutermost() != CompositeFontEditorPtr->GetFontObject()->GetOutermost())
		{
			return FontFaceAsset->GetPathName();
		}

	}
	
	return FString();
}

void STypefaceEntryEditor::OnFontFaceAssetChanged(const FAssetData& InAssetData)
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if(TypefaceEntryPtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetFontFaceAsset", "Set Font Face Asset"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		TypefaceEntryPtr->Font = FFontData(InAssetData.GetAsset());
		CompositeFontEditorPtr->FlushCachedFont();

		CacheSubFaceData();
	}
}

FReply STypefaceEntryEditor::OnBrowseTypefaceEntryFontPath()
{
	IDesktopPlatform* const DesktopPlatform = FDesktopPlatformModule::Get();

	if(DesktopPlatform)
	{
		const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);

		TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
		const void* const ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
			? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
			: nullptr;

		TArray<FString> OutFiles;
		if (DesktopPlatform->OpenFileDialog(
			ParentWindowHandle,
			LOCTEXT("FontPickerTitle", "Choose a font file...").ToString(),
			DefaultPath,
			TEXT(""),
			TEXT("All Font Files (*.ttf, *.ttc, *.otf, *.otc)|*.ttf;*.ttc;*.otf;*.otc|TrueType fonts (*.ttf, *.ttc)|*.ttf;*.ttc|OpenType fonts (*.otf, *.otc)|*.otf;*.otc"),
			EFileDialogFlags::None, 
			OutFiles
			))
		{
			OnTypefaceEntryFontPathPicked(OutFiles[0]);
		}
	}

	return FReply::Handled();
}

void STypefaceEntryEditor::OnTypefaceEntryFontPathPicked(const FString& InNewFontFilename)
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if(TypefaceEntryPtr)
	{
		UFontFace* TempFontFace = NewObject<UFontFace>();
		TempFontFace->SourceFilename = InNewFontFilename;

		TArray<uint8> TempFontFaceData;
		if (FFileHelper::LoadFileToArray(TempFontFaceData, *TempFontFace->SourceFilename))
		{
			TempFontFace->FontFaceData->SetData(MoveTemp(TempFontFaceData));

			UFontFace* NewFontFaceAsset = SaveFontFaceAsAsset(TempFontFace, *FPaths::GetBaseFilename(TempFontFace->SourceFilename));
			if (NewFontFaceAsset)
			{
				OnFontFaceAssetChanged(FAssetData(NewFontFaceAsset));
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewFontFaceAsset);
			}
		}
	}

	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InNewFontFilename));
}

FReply STypefaceEntryEditor::OnDeleteFontClicked()
{
	OnDeleteFont.ExecuteIfBound(TypefaceEntry);

	return FReply::Handled();
}

void STypefaceEntryEditor::CacheSubFaceData()
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	SubFacesData.Reset();
	if (TypefaceEntryPtr)
	{
		const UFontFace* FontFaceAsset = Cast<const UFontFace>(TypefaceEntryPtr->Font.GetFontFaceAsset());
		if (FontFaceAsset)
		{
			SubFacesData.Reserve(FontFaceAsset->SubFaces.Num());
			for (int32 SubFaceIndex = 0; SubFaceIndex < FontFaceAsset->SubFaces.Num(); ++SubFaceIndex)
			{
				TSharedPtr<FSubFaceInfo> SubFace = SubFacesData.Add_GetRef(MakeShared<FSubFaceInfo>());
				SubFace->Index = SubFaceIndex;
				SubFace->Description = FText::FromString(FontFaceAsset->SubFaces[SubFaceIndex]);
			}
		}
	}

	if (SubFacesCombo.IsValid())
	{
		SubFacesCombo->RefreshOptions();
	}
}

EVisibility STypefaceEntryEditor::GetSubFaceVisibility() const
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if (TypefaceEntryPtr)
	{
		const UFontFace* FontFaceAsset = Cast<const UFontFace>(TypefaceEntryPtr->Font.GetFontFaceAsset());

		// Only show for fonts with multiple sub-faces
		if (FontFaceAsset && FontFaceAsset->SubFaces.Num() > 1)
		{
			return EVisibility::Visible;
		}

	}

	return EVisibility::Collapsed;
}

FText STypefaceEntryEditor::GetCurrentSubFaceSelectionDisplayName() const
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if (TypefaceEntryPtr)
	{
		const int32 SubFaceIndex = TypefaceEntryPtr->Font.GetSubFaceIndex();
		if (SubFacesData.IsValidIndex(SubFaceIndex))
		{
			return SubFacesData[SubFaceIndex]->Description;
		}
	}

	return FText::GetEmpty();
}

void STypefaceEntryEditor::OnSubFaceSelectionChanged(TSharedPtr<FSubFaceInfo> InSubFace, ESelectInfo::Type)
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if (TypefaceEntryPtr && InSubFace.IsValid())
	{
		const FScopedTransaction Transaction(LOCTEXT("ChangeSubFace", "Change Sub-Face"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		TypefaceEntryPtr->Font.SetSubFaceIndex(InSubFace->Index);

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

TSharedRef<SWidget> STypefaceEntryEditor::MakeSubFaceSelectionWidget(TSharedPtr<FSubFaceInfo> InSubFace)
{
	return 
		SNew(STextBlock)
		.Text(InSubFace->Description)
		.ToolTipText(InSubFace->Description);
}

EVisibility STypefaceEntryEditor::GetUpgradeDataVisibility() const
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if (TypefaceEntryPtr)
	{
		const UFontFace* FontFaceAsset = Cast<const UFontFace>(TypefaceEntryPtr->Font.GetFontFaceAsset());

		// Only show for font faces within the same package as the main font
		if (FontFaceAsset && FontFaceAsset->GetOutermost() == CompositeFontEditorPtr->GetFontObject()->GetOutermost())
		{
			return EVisibility::Visible;
		}

	}

	return EVisibility::Collapsed;
}

FReply STypefaceEntryEditor::OnUpgradeDataClicked()
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();

	if (TypefaceEntryPtr)
	{
		const UFontFace* FontFaceAsset = Cast<const UFontFace>(TypefaceEntryPtr->Font.GetFontFaceAsset());
		check(FontFaceAsset);

		UFontFace* NewFontFaceAsset = SaveFontFaceAsAsset(FontFaceAsset, nullptr);
		if (NewFontFaceAsset)
		{
			OnFontFaceAssetChanged(FAssetData(NewFontFaceAsset));
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewFontFaceAsset);
		}
	}

	return FReply::Handled();
}

UFontFace* STypefaceEntryEditor::SaveFontFaceAsAsset(const UFontFace* InFontFace, const TCHAR* InDefaultNameOverride)
{
	const FString DefaultPackageName = CompositeFontEditorPtr->GetFontObject()->GetOutermost()->GetName();
	const FString DefaultPackagePath = FPackageName::GetLongPackagePath(DefaultPackageName);
	const FString DefaultFaceAssetName = InDefaultNameOverride ? FString(InDefaultNameOverride) : InFontFace->GetName();

	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DefaultPath = DefaultPackagePath;
	SaveAssetDialogConfig.DefaultAssetName = DefaultFaceAssetName;
	SaveAssetDialogConfig.AssetClassNames.Add(InFontFace->GetClass()->GetClassPathName());
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveFontFaceDialogTitle", "Save Font Face");

	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FString NewPackageName;
	bool bFilenameValid = false;
	while (!bFilenameValid)
	{
		NewPackageName = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (NewPackageName.IsEmpty())
		{
			bFilenameValid = false;
			break;
		}

		NewPackageName = FPackageName::ObjectPathToPackageName(NewPackageName);

		FText OutError;
		bFilenameValid = FFileHelper::IsFilenameValidForSaving(NewPackageName, OutError);
	}

	if (bFilenameValid)
	{
		const FString NewFaceAssetName = FPackageName::GetLongPackageAssetName(NewPackageName);
		UPackage* NewFaceAssetPackage = CreatePackage( *NewPackageName);

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(GetMutableDefault<UFontFileImportFactory>(), UFontFace::StaticClass(), NewFaceAssetPackage, *NewFaceAssetName, *FPaths::GetExtension(InFontFace->GetFontFilename()));

		UFontFace* NewFaceAsset = Cast<UFontFace>(StaticDuplicateObject(InFontFace, NewFaceAssetPackage, *NewFaceAssetName));
		if (NewFaceAsset)
		{
			// Make sure the new object is flagged correctly
			NewFaceAsset->SetFlags(RF_Public | RF_Standalone);

			NewFaceAsset->MarkPackageDirty();
			FAssetRegistryModule::AssetCreated(NewFaceAsset);
		}

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(GetMutableDefault<UFontFileImportFactory>(), NewFaceAsset);

		return NewFaceAsset;
	}

	return nullptr;
}

FSlateFontInfo STypefaceEntryEditor::GetPreviewFontStyle() const
{
	FTypefaceEntry* const TypefaceEntryPtr = TypefaceEntry->GetTypefaceEntry();
	return FSlateFontInfo(CompositeFontEditorPtr->GetFontObject(), 9, (TypefaceEntryPtr) ? TypefaceEntryPtr->Name : NAME_None);
}

SSubTypefaceEditor::~SSubTypefaceEditor()
{
}

void SSubTypefaceEditor::Construct(const FArguments& InArgs)
{
	CompositeFontEditorPtr = InArgs._CompositeFontEditor;
	SubTypeface = InArgs._SubTypeface;
	ParentTypeface = InArgs._ParentTypeface;
	OnDeleteSubFontFamily = InArgs._OnDeleteSubFontFamily;

	ChildSlot
	[
		SAssignNew(TypefaceEditor, STypefaceEditor)
		.CompositeFontEditor(InArgs._CompositeFontEditor)
		.Typeface(this, &SSubTypefaceEditor::GetTypeface)
		.TypefaceDisplayName(this, &SSubTypefaceEditor::GetDisplayName)
		.OnDisplayNameCommitted(this, &SSubTypefaceEditor::OnDisplayNameCommitted)
		.TypefaceDisplayNameToolTip(LOCTEXT("FontFamilyNameTooltip", "The name of this font family (click to edit)"))
		.HeaderContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CulturesLabel", "Cultures:"))
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SEditableTextBox)
						.MinDesiredWidth(20.0f)
						.ToolTipText(LOCTEXT("CulturesTooltip", "An optional semi-colon separated list of cultures that this sub-font should be used with (if specified, this sub-font will be favored by those cultures and ignored by others)"))
						.Text(this, &SSubTypefaceEditor::GetCultures)
						.OnTextCommitted(this, &SSubTypefaceEditor::OnCulturesCommitted)
					]
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(4.0f, 0.0f))
				[
					SNew(SFontScalingFactorEditor)
					.CompositeFontEditor(CompositeFontEditorPtr)
					.FallbackFont_Lambda([this] { return (SubTypeface.IsValid()) ? SubTypeface->GetSubTypefaceEntry() : nullptr; })
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SFontOverrideSelector)
					.CompositeFontEditor(CompositeFontEditorPtr)
					.TypefaceEditor_Lambda([this] { return TypefaceEditor.Get(); })
					.Typeface_Lambda([this] { return (SubTypeface.IsValid() && SubTypeface->GetSubTypefaceEntry()) ? &SubTypeface->GetSubTypefaceEntry()->Typeface : nullptr; })
					.ParentTypeface(ParentTypeface)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(8.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ToolTipText(LOCTEXT("DeleteFontFamilyTooltip", "Remove this sub-font family from the composite font"))
					.OnClicked(this, &SSubTypefaceEditor::OnDeleteSubFontFamilyClicked)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Delete"))
					]
				]
			]
		]
		.BodyContent()
		[
			SNew(SBox)
			.Padding(FMargin(8.0f, 0.0f, 8.0f, 0.0f))
			[
				SAssignNew(CharacterRangeEntriesTileView, STileView<FCharacterRangeTileViewEntryPtr>)
				.ListItemsSource(&CharacterRangeEntries)
				.SelectionMode(ESelectionMode::None)
				.ItemWidth(180)
				.ItemHeight(160)
				.ItemAlignment(EListItemAlignment::LeftAligned)
				.OnGenerateTile(this, &SSubTypefaceEditor::MakeCharacterRangesEntryWidget)
			]
		]
	];

	UpdateCharacterRangesList();
}

void SSubTypefaceEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (SubTypeface.IsValid() && SubTypeface->bRenameRequested)
	{
		SubTypeface->bRenameRequested = false;
		TypefaceEditor->RequestRename();
	}
}

FTypeface* SSubTypefaceEditor::GetTypeface() const
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();
	return (SubTypefaceEntryPtr) ? &SubTypefaceEntryPtr->Typeface : nullptr;
}

FText SSubTypefaceEditor::GetDisplayName() const
{
	const FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();
	
	if(SubTypefaceEntryPtr)
	{
		return (SubTypefaceEntryPtr->EditorName.IsNone()) 
			? FText::Format(LOCTEXT("SubFontFamilyNameFmt", "Sub-Font Family #{0}"), FText::AsNumber(SubTypeface->SubTypefaceEntryIndex + 1))
			: FText::FromName(SubTypefaceEntryPtr->EditorName);
	}

	return FText::GetEmpty();
}

void SSubTypefaceEditor::OnDisplayNameCommitted(const FText& InNewName, ETextCommit::Type InCommitType)
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();
	
	if(SubTypefaceEntryPtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetFontFamilyDisplayName", "Set Font Family Display Name"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		const FText DefaultText = FText::Format(LOCTEXT("SubFontFamilyNameFmt", "Sub-Font Family #{0}"), FText::AsNumber(SubTypeface->SubTypefaceEntryIndex + 1));
		if(InNewName.ToString().Equals(DefaultText.ToString()))
		{
			SubTypefaceEntryPtr->EditorName = NAME_None;
		}
		else
		{
			SubTypefaceEntryPtr->EditorName = *InNewName.ToString();
		}
	}
}

FReply SSubTypefaceEditor::OnDeleteSubFontFamilyClicked()
{
	OnDeleteSubFontFamily.ExecuteIfBound(SubTypeface);

	return FReply::Handled();
}

void SSubTypefaceEditor::UpdateCharacterRangesList()
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();

	for(FCharacterRangeTileViewEntryPtr& CharacterRangeTileViewEntry : CharacterRangeEntries)
	{
		CharacterRangeTileViewEntry->Reset();
	}

	CharacterRangeEntries.Empty((SubTypefaceEntryPtr) ? SubTypefaceEntryPtr->CharacterRanges.Num() : 0);

	if(SubTypefaceEntryPtr)
	{
		for(int32 CharacterRangeIndex = 0; CharacterRangeIndex < SubTypefaceEntryPtr->CharacterRanges.Num(); ++CharacterRangeIndex)
		{
			CharacterRangeEntries.Add(MakeShareable(new FCharacterRangeTileViewEntry(SubTypeface, CharacterRangeIndex)));
		}
	}

	// Add a dummy entry for the "Add" button slot
	CharacterRangeEntries.Add(MakeShareable(new FCharacterRangeTileViewEntry()));

	CharacterRangeEntriesTileView->RequestListRefresh();
}

TSharedRef<ITableRow> SSubTypefaceEditor::MakeCharacterRangesEntryWidget(FCharacterRangeTileViewEntryPtr InCharacterRangeEntry, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SWidget> EntryWidget;

	if(InCharacterRangeEntry->RangeEntryIndex == INDEX_NONE)
	{
		// Dummy entry for the "Add" button
		EntryWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("AddCharacterRangeTooltip", "Add a new character range to this sub-font family"))
			.OnClicked(this, &SSubTypefaceEditor::OnAddCharacterRangeClicked)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.0f)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Add"))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Text(LOCTEXT("AddCharacterRange", "Add Character Range"))
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.Justification(ETextJustify::Center)
				]
			]
		];
	}
	else
	{
		EntryWidget = SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(8.0f)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(SCharacterRangeEditor)
				.CompositeFontEditor(CompositeFontEditorPtr)
				.CharacterRange(InCharacterRangeEntry)
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("DeleteCharacterRangeTooltip", "Remove this character range from the sub-font family"))
				.OnClicked(this, &SSubTypefaceEditor::OnDeleteCharacterRangeClicked, InCharacterRangeEntry)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("FontEditor.Button_Delete"))
				]
			]
		];
	}

	return
		SNew(STableRow<FCharacterRangeTileViewEntryPtr>, OwnerTable)
		[
			SNew(SBox)
			.Padding(FMargin(0.0f, 0.0f, 8.0f, 8.0f))
			[
				EntryWidget.ToSharedRef()
			]
		];
}

FReply SSubTypefaceEditor::OnAddCharacterRangeClicked()
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();

	if(SubTypefaceEntryPtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddCharacterRange", "Add Character Range"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		SubTypefaceEntryPtr->CharacterRanges.Add(FInt32Range::Empty());

		UpdateCharacterRangesList();

		CompositeFontEditorPtr->FlushCachedFont();
	}

	return FReply::Handled();
}

FReply SSubTypefaceEditor::OnDeleteCharacterRangeClicked(FCharacterRangeTileViewEntryPtr InCharacterRangeEntry)
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();

	if(SubTypefaceEntryPtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("DeleteCharacterRange", "Delete Character Range"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		SubTypefaceEntryPtr->CharacterRanges.RemoveAt(InCharacterRangeEntry->RangeEntryIndex);

		UpdateCharacterRangesList();

		CompositeFontEditorPtr->FlushCachedFont();
	}

	return FReply::Handled();
}

FText SSubTypefaceEditor::GetCultures() const
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();

	if (SubTypefaceEntryPtr)
	{
		return FText::FromString(SubTypefaceEntryPtr->Cultures);
	}

	return FText::GetEmpty();
}

void SSubTypefaceEditor::OnCulturesCommitted(const FText& InCultures, ETextCommit::Type InCommitType)
{
	FCompositeSubFont* const SubTypefaceEntryPtr = SubTypeface->GetSubTypefaceEntry();

	if (SubTypefaceEntryPtr)
	{
		SubTypefaceEntryPtr->Cultures = InCultures.ToString();

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

SCharacterRangeEditor::~SCharacterRangeEditor()
{
}

void SCharacterRangeEditor::Construct(const FArguments& InArgs)
{
	CompositeFontEditorPtr = InArgs._CompositeFontEditor;
	CharacterRange = InArgs._CharacterRange;

	CacheCurrentRangeSelection();

	// Copy the data so we can sort it by display name (it's usually ordered by ascending block range, and the sort happens when opening the combo)
	TSharedPtr<FUnicodeBlockRange> CurrentRangeSelectionItem;
	{
		TArrayView<const FUnicodeBlockRange> UnicodeBlockRanges = FUnicodeBlockRange::GetUnicodeBlockRanges();
		RangeSelectionComboData.Reserve(UnicodeBlockRanges.Num());
		for(const FUnicodeBlockRange& UnicodeBlockRange : UnicodeBlockRanges)
		{
			RangeSelectionComboData.Emplace(MakeShared<FUnicodeBlockRange>(UnicodeBlockRange));

			if(CurrentRangeSelection.IsSet() && CurrentRangeSelection->GetRange() == UnicodeBlockRange.GetRange())
			{
				CurrentRangeSelectionItem = RangeSelectionComboData.Last();
			}
		}
	}

	ChildSlot
	[
		SNew(SVerticalBox)

		// Block selector
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0.0f, 2.0f))
		[
			SAssignNew(RangeSelectionCombo, SComboBox<TSharedPtr<FUnicodeBlockRange>>)
			.OptionsSource(&RangeSelectionComboData)
			.InitiallySelectedItem(CurrentRangeSelectionItem)
			.ContentPadding(FMargin(4.0, 2.0))
			.OnComboBoxOpening(this, &SCharacterRangeEditor::OnRangeSelectionComboOpening)
			.OnSelectionChanged(this, &SCharacterRangeEditor::OnRangeSelectionChanged)
			.OnGenerateWidget(this, &SCharacterRangeEditor::MakeRangeSelectionWidget)
			[
				SNew(STextBlock)
				.Text(this, &SCharacterRangeEditor::GetCurrentRangeSelectionDisplayName)
				.ToolTipText(this, &SCharacterRangeEditor::GetCurrentRangeSelectionDisplayName)
			]
		]

		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)

			// Minimum column
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SEditableTextBox)
					.Text(this, &SCharacterRangeEditor::GetRangeComponentAsTCHAR, 0)
					.OnTextCommitted(this, &SCharacterRangeEditor::OnRangeComponentCommittedAsTCHAR, 0)
					.ToolTipText(LOCTEXT("MinCharacterRangeEditCharTooltip", "Specifies the lower inclusive boundary of this character range as a literal unicode character.\nExample: If you wanted to use the range 'A-Z', this would be set to 'A'."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SEditableTextBox)
					.Text(this, &SCharacterRangeEditor::GetRangeComponentAsHexString, 0)
					.OnTextCommitted(this, &SCharacterRangeEditor::OnRangeComponentCommittedAsHexString, 0)
					.ToolTipText(LOCTEXT("MinCharacterRangeEditHexTooltip", "Specifies the lower inclusive boundary of this character range as the hexadecimal value of a unicode character.\nExample: If you wanted to use the range '0x41-0x5A' (A-Z), this would be set to '0x41'."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SNumericEntryBox<int32>)
					.Value(this, &SCharacterRangeEditor::GetRangeComponentAsOptional, 0)
					.OnValueCommitted(this, &SCharacterRangeEditor::OnRangeComponentCommittedAsNumeric, 0)
					.ToolTipText(LOCTEXT("MinCharacterRangeEditDecTooltip", "Specifies the lower inclusive boundary of this character range as the decimal value of a unicode character.\nExample: If you wanted to use the range '65-90' (A-Z), this would be set to '65'."))
				]
			]

			// Separator
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::AsCultureInvariant(TEXT(" - ")))
				.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			]

			// Maximum column
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SEditableTextBox)
					.Text(this, &SCharacterRangeEditor::GetRangeComponentAsTCHAR, 1)
					.OnTextCommitted(this, &SCharacterRangeEditor::OnRangeComponentCommittedAsTCHAR, 1)
					.ToolTipText(LOCTEXT("MaxCharacterRangeEditCharTooltip", "Specifies the upper inclusive boundary of this character range as a literal unicode character.\nExample: If you wanted to use the range 'A-Z', this would be set to 'Z'."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SEditableTextBox)
					.Text(this, &SCharacterRangeEditor::GetRangeComponentAsHexString, 1)
					.OnTextCommitted(this, &SCharacterRangeEditor::OnRangeComponentCommittedAsHexString, 1)
					.ToolTipText(LOCTEXT("MaxCharacterRangeEditHexTooltip", "Specifies the upper inclusive boundary of this character range as the hexadecimal value of a unicode character.\nExample: If you wanted to use the range '0x41-0x5A' (A-Z), this would be set to '0x5A'."))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(SNumericEntryBox<int32>)
					.Value(this, &SCharacterRangeEditor::GetRangeComponentAsOptional, 1)
					.OnValueCommitted(this, &SCharacterRangeEditor::OnRangeComponentCommittedAsNumeric, 1)
					.ToolTipText(LOCTEXT("MaxCharacterRangeEditDecTooltip", "Specifies the upper inclusive boundary of this character range as the decimal value of a unicode character.\nExample: If you wanted to use the range '65-90' (A-Z), this would be set to '90'."))
				]
			]
		]
	];
}

FText SCharacterRangeEditor::GetRangeComponentAsTCHAR(int32 InComponentIndex) const
{
	const int32 RangeComponent = GetRangeComponent(InComponentIndex);
	const TCHAR RangeComponentStr[] = { static_cast<TCHAR>(RangeComponent), 0 };
	return FText::AsCultureInvariant(RangeComponentStr);
}

FText SCharacterRangeEditor::GetRangeComponentAsHexString(int32 InComponentIndex) const
{
	const int32 RangeComponent = GetRangeComponent(InComponentIndex);
	return FText::AsCultureInvariant(FString::Printf(TEXT("0x%04x"), RangeComponent));
}

TOptional<int32> SCharacterRangeEditor::GetRangeComponentAsOptional(int32 InComponentIndex) const
{
	return GetRangeComponent(InComponentIndex);
}

int32 SCharacterRangeEditor::GetRangeComponent(const int32 InComponentIndex) const
{
	check(InComponentIndex == 0 || InComponentIndex == 1);

	const FInt32Range* const CharacterRangePtr = CharacterRange->GetRange();
	if(CharacterRangePtr)
	{
		return (InComponentIndex == 0) ? CharacterRangePtr->GetLowerBoundValue() : CharacterRangePtr->GetUpperBoundValue();
	}

	return 0;
}

void SCharacterRangeEditor::OnRangeComponentCommittedAsTCHAR(const FText& InNewValue, ETextCommit::Type InCommitType, int32 InComponentIndex)
{
	const FString NewValueStr = InNewValue.ToString();
	if(NewValueStr.Len() == 1)
	{
		SetRangeComponent(static_cast<int32>(NewValueStr[0]), InComponentIndex);
	}
	else if(NewValueStr.Len() == 0)
	{
		SetRangeComponent(0, InComponentIndex);
	}
}

void SCharacterRangeEditor::OnRangeComponentCommittedAsHexString(const FText& InNewValue, ETextCommit::Type InCommitType, int32 InComponentIndex)
{
	const FString NewValueStr = InNewValue.ToString();
	const TCHAR* HexStart = NewValueStr.GetCharArray().GetData();
	if(NewValueStr.StartsWith(TEXT("0x")))
	{
		// Skip the "0x" part, as FParse::HexNumber doesn't handle that
		HexStart += 2;
	}

	const int32 NewValue = FParse::HexNumber(HexStart);
	SetRangeComponent(NewValue, InComponentIndex);
}

void SCharacterRangeEditor::OnRangeComponentCommittedAsNumeric(int32 InNewValue, ETextCommit::Type InCommitType, int32 InComponentIndex)
{
	SetRangeComponent(InNewValue, InComponentIndex);
}

void SCharacterRangeEditor::SetRangeComponent(const int32 InNewValue, const int32 InComponentIndex)
{
	check(InComponentIndex == 0 || InComponentIndex == 1);
	
	FInt32Range* const CharacterRangePtr = CharacterRange->GetRange();
	if(CharacterRangePtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("UpdateCharacterRange", "Update Character Range"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		*CharacterRangePtr = (InComponentIndex == 0) 
			? FInt32Range(FInt32Range::BoundsType::Inclusive(InNewValue), FInt32Range::BoundsType::Inclusive(CharacterRangePtr->GetUpperBoundValue())) 
			: FInt32Range(FInt32Range::BoundsType::Inclusive(CharacterRangePtr->GetLowerBoundValue()), FInt32Range::BoundsType::Inclusive(InNewValue));

		CacheCurrentRangeSelection();

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

void SCharacterRangeEditor::CacheCurrentRangeSelection()
{
	CurrentRangeSelection.Reset();

	TArrayView<const FUnicodeBlockRange> UnicodeBlockRanges = FUnicodeBlockRange::GetUnicodeBlockRanges();

	// todo: could binary search on the lower bound since they're sorted in ascending order; need the Algo for it to come back from Main
	FInt32Range* const CharacterRangePtr = CharacterRange->GetRange();
	if(CharacterRangePtr)
	{
		for(const FUnicodeBlockRange& UnicodeBlockRange : UnicodeBlockRanges)
		{
			if(UnicodeBlockRange.GetRange() == *CharacterRangePtr)
			{
				CurrentRangeSelection = UnicodeBlockRange;
			}
		}
	}
}

FText SCharacterRangeEditor::GetCurrentRangeSelectionDisplayName() const
{
	return CurrentRangeSelection.IsSet() ? CurrentRangeSelection->GetDisplayName() : LOCTEXT("UnicodeBlock_CustomSelection", "Custom");
}

void SCharacterRangeEditor::OnRangeSelectionComboOpening()
{
	RangeSelectionComboData.Sort([](const TSharedPtr<FUnicodeBlockRange>& One, const TSharedPtr<FUnicodeBlockRange>& Two)
	{
		return One->GetDisplayName().CompareTo(Two->GetDisplayName()) < 0;
	});

	if(RangeSelectionCombo.IsValid())
	{
		RangeSelectionCombo->RefreshOptions();
	}
}

void SCharacterRangeEditor::OnRangeSelectionChanged(TSharedPtr<FUnicodeBlockRange> InNewRangeSelection, ESelectInfo::Type)
{
	if(InNewRangeSelection.IsValid())
	{
		FInt32Range* const CharacterRangePtr = CharacterRange->GetRange();
		if(CharacterRangePtr)
		{
			const FScopedTransaction Transaction(LOCTEXT("UpdateCharacterRange", "Update Character Range"));
			CompositeFontEditorPtr->GetFontObject()->Modify();

			*CharacterRangePtr = InNewRangeSelection->GetRange();
			CurrentRangeSelection = *InNewRangeSelection;

			CompositeFontEditorPtr->FlushCachedFont();
		}
	}
}

TSharedRef<SWidget> SCharacterRangeEditor::MakeRangeSelectionWidget(TSharedPtr<FUnicodeBlockRange> InRangeSelection)
{
	return SNew(STextBlock)
		.Text(InRangeSelection->GetDisplayName())
		.ToolTipText(FText::Format(LOCTEXT("RangeSelectionTooltipFmt", "{0} ({1})"), InRangeSelection->GetDisplayName(), FText::AsCultureInvariant(FString::Printf(TEXT("0x%04x - 0x%04x"), InRangeSelection->RangeLower, InRangeSelection->RangeUpper))));
}

void SFontScalingFactorEditor::Construct(const FArguments& InArgs)
{
	CompositeFontEditorPtr = InArgs._CompositeFontEditor;
	FallbackFont = InArgs._FallbackFont;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.0f, 0.0f, 2.0f, 0.0f))
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ScalingFactorLabel", "Scaling Factor:"))
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SNumericEntryBox<float>)
			.ToolTipText(LOCTEXT("ScalingFactorTooltip", "The scaling factor will adjust the size of the rendered glyphs so that you can tweak their size to match that of the default font family\nNote: Only applies to scalable font formats (ie, not to bitmap fonts)"))
			.Value(this, &SFontScalingFactorEditor::GetScalingFactorAsOptional)
			.OnValueCommitted(this, &SFontScalingFactorEditor::OnScalingFactorCommittedAsNumeric)
		]
	];
}

TOptional<float> SFontScalingFactorEditor::GetScalingFactorAsOptional() const
{
	const FCompositeFallbackFont* const FallbackFontPtr = FallbackFont.Get(nullptr);

	if(FallbackFontPtr)
	{
		return FallbackFontPtr->ScalingFactor;
	}

	return TOptional<float>();
}

void SFontScalingFactorEditor::OnScalingFactorCommittedAsNumeric(float InNewValue, ETextCommit::Type InCommitType)
{
	FCompositeFallbackFont* const FallbackFontPtr = FallbackFont.Get(nullptr);

	if(FallbackFontPtr)
	{
		const FScopedTransaction Transaction(LOCTEXT("SetScalingFactor", "Set Scaling Factor"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		FallbackFontPtr->ScalingFactor = InNewValue;

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

void SFontOverrideSelector::Construct(const FArguments& InArgs)
{
	CompositeFontEditorPtr = InArgs._CompositeFontEditor;
	TypefaceEditor = InArgs._TypefaceEditor;
	Typeface = InArgs._Typeface;
	ParentTypeface = InArgs._ParentTypeface;

	ChildSlot
	[
		SAssignNew(FontOverrideCombo, SComboBox<TSharedPtr<FName>>)
		.OptionsSource(&FontOverrideComboData)
		.ContentPadding(FMargin(4.0, 2.0))
		.Visibility(this, &SFontOverrideSelector::GetAddFontOverrideVisibility)
		.IsEnabled(this, &SFontOverrideSelector::IsFontOverrideComboEnabled)
		.OnComboBoxOpening(this, &SFontOverrideSelector::OnAddFontOverrideComboOpening)
		.OnSelectionChanged(this, &SFontOverrideSelector::OnAddFontOverrideSelectionChanged)
		.OnGenerateWidget(this, &SFontOverrideSelector::MakeAddFontOverrideWidget)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AddFontOverride", "Add Font Override"))
			.ToolTipText(LOCTEXT("AddFontOverrideTooltip", "Override a font from the default font family to ensure it will be used when drawing a glyph in the range of this sub-font family"))
		]
	];
}

EVisibility SFontOverrideSelector::GetAddFontOverrideVisibility() const
{
	return (ParentTypeface.Get(nullptr)) ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SFontOverrideSelector::IsFontOverrideComboEnabled() const
{
	FTypeface* const TypefacePtr = Typeface.Get(nullptr);
	const FTypeface* const ParentTypefacePtr = ParentTypeface.Get(nullptr);

	if (TypefacePtr && ParentTypefacePtr)
	{
		// Check whether our parent font has any entries that haven't already got a local entry
		for (const FTypefaceEntry& ParentTypefaceEntry : ParentTypefacePtr->Fonts)
		{
			const bool bIsOverridden = TypefacePtr->Fonts.ContainsByPredicate([&ParentTypefaceEntry](const FTypefaceEntry& LocalTypefaceEntry)
			{
				return LocalTypefaceEntry.Name == ParentTypefaceEntry.Name;
			});

			if (!bIsOverridden)
			{
				return true;
			}
		}
	}

	return false;
}

void SFontOverrideSelector::OnAddFontOverrideComboOpening()
{
	FontOverrideComboData.Reset();

	FTypeface* const TypefacePtr = Typeface.Get(nullptr);
	const FTypeface* const ParentTypefacePtr = ParentTypeface.Get(nullptr);

	if(TypefacePtr && ParentTypefacePtr)
	{
		TSet<FName> LocalFontNames;
		for(const FTypefaceEntry& LocalTypefaceEntry : TypefacePtr->Fonts)
		{
			LocalFontNames.Add(LocalTypefaceEntry.Name);
		}

		// Add every font from our parent font that hasn't already got a local entry
		for(const FTypefaceEntry& ParentTypefaceEntry : ParentTypefacePtr->Fonts)
		{
			if(!LocalFontNames.Contains(ParentTypefaceEntry.Name))
			{
				FontOverrideComboData.Add(MakeShareable(new FName(ParentTypefaceEntry.Name)));
			}
		}
	}

	FontOverrideCombo->RefreshOptions();
}

void SFontOverrideSelector::OnAddFontOverrideSelectionChanged(TSharedPtr<FName> InNewSelection, ESelectInfo::Type)
{
	FTypeface* const TypefacePtr = Typeface.Get(nullptr);

	if(TypefacePtr && InNewSelection.IsValid() && !InNewSelection->IsNone())
	{
		const FScopedTransaction Transaction(LOCTEXT("AddFontOverride", "Add Font Override"));
		CompositeFontEditorPtr->GetFontObject()->Modify();

		TypefacePtr->Fonts.Add(FTypefaceEntry(*InNewSelection));

		if(STypefaceEditor* TypefaceEditorPtr = TypefaceEditor.Get(nullptr))
		{
			TypefaceEditorPtr->Refresh();
		}

		CompositeFontEditorPtr->FlushCachedFont();
	}
}

TSharedRef<SWidget> SFontOverrideSelector::MakeAddFontOverrideWidget(TSharedPtr<FName> InFontEntry)
{
	return
		SNew(STextBlock)
		.Text(FText::FromName(*InFontEntry));
}

#undef LOCTEXT_NAMESPACE
