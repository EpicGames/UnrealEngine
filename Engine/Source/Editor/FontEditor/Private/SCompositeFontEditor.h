// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Fonts/SlateFontInfo.h"
#include "Fonts/UnicodeBlockRange.h"
#include "Framework/SlateDelegates.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Types/SlateConstants.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STileView.h"

class IFontEditor;
class ITableRow;
class SInlineEditableTextBlock;
class SWidget;
class UFont;
class UFontFace;
struct FAssetData;
struct FCharacterRangeTileViewEntry;
struct FCompositeFallbackFont;
struct FCompositeFont;
struct FGeometry;
struct FSubTypefaceListViewEntry;
struct FTypeface;
struct FTypefaceListViewEntry;

typedef TSharedPtr<FTypefaceListViewEntry> FTypefaceListViewEntryPtr;

typedef TSharedPtr<FSubTypefaceListViewEntry> FSubTypefaceListViewEntryPtr;

typedef TSharedPtr<FCharacterRangeTileViewEntry> FCharacterRangeTileViewEntryPtr;

class SCompositeFontEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCompositeFontEditor)
		: _FontEditor()
		{}

		/** Pointer back to the Font editor tool that owns us */
		SLATE_ARGUMENT(TWeakPtr<IFontEditor>, FontEditor)

	SLATE_END_ARGS()

	/** Destructor */
	~SCompositeFontEditor();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);

	/** Refresh the editor in response to an external change */
	void Refresh();

	/** Flush the font cache for the current font */
	void FlushCachedFont();

	/** Get the root font object being edited */
	UFont* GetFontObject() const;

private:
	/** Get the composite font being edited */
	FCompositeFont* GetCompositeFont() const;

	/** Get the default typeface */
	FTypeface* GetDefaultTypeface() const;

	/** Get the default typeface */
	const FTypeface* GetConstDefaultTypeface() const;

	/** Get the fallback typeface */
	FTypeface* GetFallbackTypeface() const;

	/** Get the fallback typeface */
	const FTypeface* GetConstFallbackTypeface() const;

	/** Get the fallback font */
	FCompositeFallbackFont* GetFallbackFont() const;

	/** Update the list of sub-typefaces in this composite font */
	void UpdateSubTypefaceList();

	/** Make the widget for an entry in the sub-typeface entries list view */
	TSharedRef<ITableRow> MakeSubTypefaceEntryWidget(FSubTypefaceListViewEntryPtr InSubTypefaceEntry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called in response to the "Add Sub-Font Family" button being clicked" */
	FReply OnAddSubFontFamily();

	/** Delete the given sub-font family from this composite font */
	void OnDeleteSubFontFamily(const FSubTypefaceListViewEntryPtr& SubTypefaceEntryToRemove);

	/** Pointer back to the Font editor tool that owns us */
	TWeakPtr<IFontEditor> FontEditorPtr;

	/** Widget for editing the default typeface */
	TSharedPtr<class STypefaceEditor> DefaultTypefaceEditor;

	/** Widget for editing the fallback typeface */
	TSharedPtr<class STypefaceEditor> FallbackTypefaceEditor;

	/** Internal list of sub-typeface pointers for the list view (generated from CompositeFontPtr->SubTypefaces) */
	TArray<FSubTypefaceListViewEntryPtr> SubTypefaceEntries;

	/** List view widget showing the sub-typeface editors (uses SubTypefaceEntries as its source) */
	TSharedPtr<SListView<FSubTypefaceListViewEntryPtr>> SubTypefaceEntriesListView;
};

class STypefaceEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STypefaceEditor)
		: _CompositeFontEditor(nullptr)
		, _Typeface(nullptr)
		, _TypefaceDisplayName()
		, _OnDisplayNameCommitted()
		, _TypefaceDisplayNameToolTip()
		, _HeaderContent()
		, _BodyContent()
		{}

		/** Pointer back to the composite font editor that owns us */
		SLATE_ARGUMENT(SCompositeFontEditor*, CompositeFontEditor)

		/** Typeface to edit (may be invalid, or change in response to an undo/redo) */
		SLATE_ATTRIBUTE(FTypeface*, Typeface)

		/** Name to show in the UI for this typeface */
		SLATE_ATTRIBUTE(FText, TypefaceDisplayName)

		/** Callback to use when the display name is committed (if not set, the display name will be read-only) */
		SLATE_EVENT(FOnTextCommitted, OnDisplayNameCommitted)

		/** Tooltip to show in the display name UI for this typeface */
		SLATE_ATTRIBUTE(FText, TypefaceDisplayNameToolTip)

		/** Slot for extra content to place in the header bar (optional) */
		SLATE_NAMED_SLOT(FArguments, HeaderContent)

		/** Slot for extra content to place above the typeface entry list (optional) */
		SLATE_NAMED_SLOT(FArguments, BodyContent)

	SLATE_END_ARGS()

	/** Destructor */
	~STypefaceEditor();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);

	/** Refresh the editor in response to an external change */
	void Refresh();

	/** Request that we begin editing the display name */
	void RequestRename();

private:
	/** Update the list of fonts in this typeface */
	void UpdateFontList();

	/** Make the widget for an entry in the typeface entries list view */
	TSharedRef<ITableRow> MakeTypefaceEntryWidget(FTypefaceListViewEntryPtr InTypefaceEntry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called in response to the "Add Font" button being clicked" */
	FReply OnAddFont();

	/** Delete the given entry from this typeface */
	void OnDeleteFont(const FTypefaceListViewEntryPtr& TypefaceEntryToRemove);

	/** Verify that the name of the given typeface entry is valid */
	bool OnVerifyFontName(const FTypefaceListViewEntryPtr& TypefaceEntryBeingRenamed, const FName& NewName, FText& OutFailureReason) const;

	/** Pointer back to the composite font editor that owns us */
	SCompositeFontEditor* CompositeFontEditorPtr;

	/** Typeface to edit (may be invalid, or change in response to an undo/redo) */
	TAttribute<FTypeface*> Typeface;

	/** Internal list of font pointers for the list view (generated from TypefacePtr->Fonts) */
	TArray<FTypefaceListViewEntryPtr> TypefaceEntries;

	/** Inline editable text for the typeface display name */
	TSharedPtr<SInlineEditableTextBlock> NameEditableTextBox;

	/** Tile view widget showing the font entries (uses TypefaceEntries as its source) */
	TSharedPtr<STileView<FTypefaceListViewEntryPtr>> TypefaceEntriesTileView;
};

class STypefaceEntryEditor : public SCompoundWidget
{
public:
	/** Declares a delegate that is executed when this typeface entry should be deleted */
	DECLARE_DELEGATE_OneParam(FOnDeleteFont, const FTypefaceListViewEntryPtr& /*TypefaceEntryToRemove*/);

	/** Declares a delegate that is executed when this typeface entry name is changed - used to verify that the new name is valid */
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnVerifyFontName, const FTypefaceListViewEntryPtr& /*TypefaceEntryBeingRenamed*/, const FName& /*NewName*/, FText& /*OutFailureReason*/);

	SLATE_BEGIN_ARGS(STypefaceEntryEditor)
		: _CompositeFontEditor(nullptr)
		, _TypefaceEntry()
		, _OnDeleteFont()
		, _OnVerifyFontName()
		{}

		/** Pointer back to the composite font editor that owns us */
		SLATE_ARGUMENT(SCompositeFontEditor*, CompositeFontEditor)

		/** Typeface entry to edit (may be invalid, or change in response to an undo/redo) */
		SLATE_ARGUMENT(FTypefaceListViewEntryPtr, TypefaceEntry)

		/** Called when this typeface entry should be deleted */
		SLATE_EVENT(FOnDeleteFont, OnDeleteFont)

		/** Called when this typeface entry name is changed - used to verify that the new name is valid */
		SLATE_EVENT(FOnVerifyFontName, OnVerifyFontName)

	SLATE_END_ARGS()

	/** Destructor */
	~STypefaceEntryEditor();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	struct FSubFaceInfo
	{
		int32 Index;
		FText Description;
	};

	/** Get the current name of this typeface entry */
	FText GetTypefaceEntryName() const;

	/** Set the current name of this typeface entry */
	void OnTypefaceEntryNameCommitted(const FText& InNewName, ETextCommit::Type InCommitType);

	/** Verify the given typename entry name is valid */
	bool OnTypefaceEntryChanged(const FText& InNewName, FText& OutFailureReason) const;

	/** Get the path to the font face asset used by this typeface entry */
	FString GetFontFaceAssetPath() const;

	/** Update the font face asset used by this typeface entry */
	void OnFontFaceAssetChanged(const FAssetData& InAssetData);

	/** Open a file picker to let you pick a new font file */
	FReply OnBrowseTypefaceEntryFontPath();

	/** Set the current font filename (and associated font data) */
	void OnTypefaceEntryFontPathPicked(const FString& InNewFontFilename);

	/** Called in response to the "Delete Font" button being clicked" */
	FReply OnDeleteFontClicked();

	/** Cache the current sub-face data for the selected font face */
	void CacheSubFaceData();

	/** Should the sub-faces combo be visible? */
	EVisibility GetSubFaceVisibility() const;

	/** Get the display name of the current sub-face selection */
	FText GetCurrentSubFaceSelectionDisplayName() const;

	/** Called when the selection of the sub-face selection combo is changed */
	void OnSubFaceSelectionChanged(TSharedPtr<FSubFaceInfo> InSubFace, ESelectInfo::Type);

	/** Make the widget for an entry in the range selection combo */
	TSharedRef<SWidget> MakeSubFaceSelectionWidget(TSharedPtr<FSubFaceInfo> InSubFace);

	/** Should the "Upgrade Data" button be visible? */
	EVisibility GetUpgradeDataVisibility() const;

	/** Called in response to the "Upgrade Data" button being clicked" */
	FReply OnUpgradeDataClicked();

	/** Saves the given font face as a real asset, and returns the asset font face instance */
	UFontFace* SaveFontFaceAsAsset(const UFontFace* InFontFace, const TCHAR* InDefaultNameOverride);

	/** Get the current font style to use for the preview text */
	FSlateFontInfo GetPreviewFontStyle() const;

	/** Pointer back to the composite font editor that owns us */
	SCompositeFontEditor* CompositeFontEditorPtr;

	/** Typeface entry to edit (may be invalid, or change in response to an undo/redo) */
	FTypefaceListViewEntryPtr TypefaceEntry;

	/** Holds a delegate that is executed when this typeface entry should be deleted */
	FOnDeleteFont OnDeleteFont;

	/** Holds a delegate that is executed when this typeface entry name is changed - used to verify that the new name is valid */
	FOnVerifyFontName OnVerifyFontName;

	/** Inline editable text for the font name */
	TSharedPtr<SInlineEditableTextBlock> NameEditableTextBox;

	/** Sub-face selection combo box widget */
	TSharedPtr<SComboBox<TSharedPtr<FSubFaceInfo>>> SubFacesCombo;

	/** Source data for the sub-faces combo box */
	TArray<TSharedPtr<FSubFaceInfo>> SubFacesData;
};

class SSubTypefaceEditor : public SCompoundWidget
{
public:
	/** Declares a delegate that is executed when this sub-typeface entry should be deleted */
	DECLARE_DELEGATE_OneParam(FOnDeleteSubFontFamily, const FSubTypefaceListViewEntryPtr& /*SubTypefaceEntryToRemove*/);

	SLATE_BEGIN_ARGS(SSubTypefaceEditor)
		: _CompositeFontEditor(nullptr)
		, _SubTypeface()
		, _ParentTypeface(nullptr)
		, _OnDeleteSubFontFamily()
		{}

		/** Pointer back to the composite font editor that owns us */
		SLATE_ARGUMENT(SCompositeFontEditor*, CompositeFontEditor)

		/** Sub-typeface to edit (may be invalid, or change in response to an undo/redo) */
		SLATE_ARGUMENT(FSubTypefaceListViewEntryPtr, SubTypeface)

		/** Parent typeface to inherit font slots from (may be invalid, or change in response to an undo/redo) */
		SLATE_ATTRIBUTE(const FTypeface*, ParentTypeface)

		/** Called when this typeface entry should be deleted */
		SLATE_EVENT(FOnDeleteSubFontFamily, OnDeleteSubFontFamily)

	SLATE_END_ARGS()

	/** Destructor */
	~SSubTypefaceEditor();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	/** Get the typeface used by this sub-typeface */
	FTypeface* GetTypeface() const;

	/** Get the display name of this sub-font family */
	FText GetDisplayName() const;

	/** Set the display name of this sub-font family */
	void OnDisplayNameCommitted(const FText& InNewName, ETextCommit::Type InCommitType);

	/** Called in response to the "Delete Sub-Font Family" button being clicked" */
	FReply OnDeleteSubFontFamilyClicked();

	/** Update the list of character ranges for this sub-font */
	void UpdateCharacterRangesList();

	/** Make the widget for an entry in the character ranges tile view */
	TSharedRef<ITableRow> MakeCharacterRangesEntryWidget(FCharacterRangeTileViewEntryPtr InCharacterRangeEntry, const TSharedRef<STableViewBase>& OwnerTable);

	/** Called in response to the "Add Character Range" button being clicked" */
	FReply OnAddCharacterRangeClicked();

	/** Called in response to the "Delete Character Range" button being clicked" */
	FReply OnDeleteCharacterRangeClicked(FCharacterRangeTileViewEntryPtr InCharacterRangeEntry);

	/** Get the semi-colon separated cultures list */
	FText GetCultures() const;

	/** Set the semi-colon separated cultures list */
	void OnCulturesCommitted(const FText& InCultures, ETextCommit::Type InCommitType);

	/** Pointer back to the composite font editor that owns us */
	SCompositeFontEditor* CompositeFontEditorPtr;

	/** Sub-typeface to edit (may be invalid, or change in response to an undo/redo) */
	FSubTypefaceListViewEntryPtr SubTypeface;

	/** Parent typeface to inherit font slots from (may be invalid, or change in response to an undo/redo) */
	TAttribute<const FTypeface*> ParentTypeface;

	/** Called when this typeface entry should be deleted */
	FOnDeleteSubFontFamily OnDeleteSubFontFamily;

	/** Nested typeface editor widget */
	TSharedPtr<STypefaceEditor> TypefaceEditor;

	/** Internal list of sub-typeface pointers for the list view (generated from SubTypeface->CharacterRanges) */
	TArray<FCharacterRangeTileViewEntryPtr> CharacterRangeEntries;

	/** Tile view widget showing the character range editors (uses CharacterRangeEntries as its source) */
	TSharedPtr<STileView<FCharacterRangeTileViewEntryPtr>> CharacterRangeEntriesTileView;
};

class SCharacterRangeEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCharacterRangeEditor)
		: _CompositeFontEditor(nullptr)
		, _CharacterRange()
		{}

		/** Pointer back to the composite font editor that owns us */
		SLATE_ARGUMENT(SCompositeFontEditor*, CompositeFontEditor)

		/** Character range to edit (may be invalid, or change in response to an undo/redo) */
		SLATE_ARGUMENT(FCharacterRangeTileViewEntryPtr, CharacterRange)

	SLATE_END_ARGS()

	/** Destructor */
	~SCharacterRangeEditor();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);

private:
	/** Get the given range component in its TCHAR form (0 for min, 1 for max) */
	FText GetRangeComponentAsTCHAR(int32 InComponentIndex) const;

	/** Get the given range component in its numerical hex form (0 for min, 1 for max) */
	FText GetRangeComponentAsHexString(int32 InComponentIndex) const;

	/** Get the given range component in its numeric form (0 for min, 1 for max) */
	TOptional<int32> GetRangeComponentAsOptional(int32 InComponentIndex) const;

	/** Get the given range component in its numeric form (0 for min, 1 for max) */
	int32 GetRangeComponent(const int32 InComponentIndex) const;

	/** Set the the given range component from its TCHAR form (0 for min, 1 for max) */
	void OnRangeComponentCommittedAsTCHAR(const FText& InNewValue, ETextCommit::Type InCommitType, int32 InComponentIndex);

	/** Set the the given range component from its numerical hex form (0 for min, 1 for max) */
	void OnRangeComponentCommittedAsHexString(const FText& InNewValue, ETextCommit::Type InCommitType, int32 InComponentIndex);

	/** Set the the given range component from its numerical form (0 for min, 1 for max) */
	void OnRangeComponentCommittedAsNumeric(int32 InNewValue, ETextCommit::Type InCommitType, int32 InComponentIndex);

	/** Set the given range component from its numeric form (0 for min, 1 for max) */
	void SetRangeComponent(const int32 InNewValue, const int32 InComponentIndex);

	/** Cache the current range selection (calculated based on the current range) */
	void CacheCurrentRangeSelection();

	/** Get the display name of the current range selection (or "Custom" if there is no range selection) */
	FText GetCurrentRangeSelectionDisplayName() const;

	/** Called before the range selection combo is opened - used to sort the list of available range selections */
	void OnRangeSelectionComboOpening();

	/** Called when the selection of the range selection combo is changed */
	void OnRangeSelectionChanged(TSharedPtr<FUnicodeBlockRange> InNewRangeSelection, ESelectInfo::Type);

	/** Make the widget for an entry in the range selection combo */
	TSharedRef<SWidget> MakeRangeSelectionWidget(TSharedPtr<FUnicodeBlockRange> InRangeSelection);

	/** Pointer back to the composite font editor that owns us */
	SCompositeFontEditor* CompositeFontEditorPtr;

	/** Character range to edit (may be invalid, or change in response to an undo/redo) */
	FCharacterRangeTileViewEntryPtr CharacterRange;

	/** Range selection combo box widget */
	TSharedPtr<SComboBox<TSharedPtr<FUnicodeBlockRange>>> RangeSelectionCombo;

	/** Source data for the range selection combo widget */
	TArray<TSharedPtr<FUnicodeBlockRange>> RangeSelectionComboData;

	/** The currently selected range selection preset */
	TOptional<FUnicodeBlockRange> CurrentRangeSelection;
};

class SFontScalingFactorEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFontScalingFactorEditor)
		: _CompositeFontEditor(nullptr)
		, _FallbackFont(nullptr)
		{}

		/** Pointer back to the composite font editor that owns us */
		SLATE_ARGUMENT(SCompositeFontEditor*, CompositeFontEditor)

		/** Fallback font that we should edit the scaling factor of (may be invalid, or change in response to an undo/redo) */
		SLATE_ATTRIBUTE(FCompositeFallbackFont*, FallbackFont)

	SLATE_END_ARGS()

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);

private:
	/** Get the scaling factor in its numeric form */
	TOptional<float> GetScalingFactorAsOptional() const;

	/** Set the the scaling factor from its numerical form */
	void OnScalingFactorCommittedAsNumeric(float InNewValue, ETextCommit::Type InCommitType);

	/** Pointer back to the composite font editor that owns us */
	SCompositeFontEditor* CompositeFontEditorPtr;

	/** Fallback font that we should edit the scaling factor of (may be invalid, or change in response to an undo/redo) */
	TAttribute<FCompositeFallbackFont*> FallbackFont;
};

class SFontOverrideSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFontOverrideSelector)
		: _CompositeFontEditor(nullptr)
		, _TypefaceEditor(nullptr)
		, _Typeface(nullptr)
		, _ParentTypeface(nullptr)
		{}

		/** Pointer back to the composite font editor that owns us */
		SLATE_ARGUMENT(SCompositeFontEditor*, CompositeFontEditor)

		/** Pointer back to the typeface editor that owns us */
		SLATE_ATTRIBUTE(STypefaceEditor*, TypefaceEditor)

		/** Typeface that we should edit the fonts on (may be invalid, or change in response to an undo/redo) */
		SLATE_ATTRIBUTE(FTypeface*, Typeface)

		/** Parent typeface to override font slots from (may be invalid, or change in response to an undo/redo) */
		SLATE_ATTRIBUTE(const FTypeface*, ParentTypeface)

	SLATE_END_ARGS()

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);

private:
	/** Get the visibility of the "Add Font Override" combo button */
	EVisibility GetAddFontOverrideVisibility() const;

	/** True if the font override combo should be enabled (due to still having parent fonts that can be overridden in this sub-font) */
	bool IsFontOverrideComboEnabled() const;

	/** Called before the font override combo is opened - used to update the list of available font overrides */
	void OnAddFontOverrideComboOpening();

	/** Called when the selection of the font override combo is changed */
	void OnAddFontOverrideSelectionChanged(TSharedPtr<FName> InNewSelection, ESelectInfo::Type);

	/** Make the widget for an entry in the font override combo */
	TSharedRef<SWidget> MakeAddFontOverrideWidget(TSharedPtr<FName> InFontEntry);

	/** Pointer back to the composite font editor that owns us */
	SCompositeFontEditor* CompositeFontEditorPtr;

	/** Pointer back to the typeface editor that owns us */
	TAttribute<STypefaceEditor*> TypefaceEditor;

	/** Typeface that we should edit the fonts on (may be invalid, or change in response to an undo/redo) */
	TAttribute<FTypeface*> Typeface;

	/** Parent typeface to override font slots from (may be invalid, or change in response to an undo/redo) */
	TAttribute<const FTypeface*> ParentTypeface;

	/** Font override combo box widget */
	TSharedPtr<SComboBox<TSharedPtr<FName>>> FontOverrideCombo;

	/** Source data for the font override combo widget */
	TArray<TSharedPtr<FName>> FontOverrideComboData;
};
