// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFontEditorViewport.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/Font.h"
#include "Engine/FontImportOptions.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Fonts/CompositeFont.h"
#include "Fonts/FontCache.h"
#include "Fonts/ShapedTextFwd.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "GenericPlatform/GenericWindow.h"
#include "HitProxies.h"
#include "IFontEditor.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Children.h"
#include "Math/Box2D.h"
#include "Math/Color.h"
#include "Math/IntPoint.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Rendering/SlateRenderer.h"
#include "SceneTypes.h"
#include "Selection.h"
#include "Slate/SceneViewport.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "TextureResource.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "ThumbnailRendering/ThumbnailRenderer.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UnrealClient.h"
#include "UnrealEdGlobals.h"
#include "ViewportClient.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SViewport.h"
#include "Widgets/SWindow.h"

#define LOCTEXT_NAMESPACE "FontEditor"


/*-----------------------------------------------------------------------------
   FFontEditorViewportClient
-----------------------------------------------------------------------------*/

class FFontEditorViewportClient : public FCommonViewportClient
{
public:
	/** Constructor */
	FFontEditorViewportClient(TWeakPtr<SFontEditorViewport> InFontEditorViewport);
	virtual ~FFontEditorViewportClient() override;

	/** FLevelEditorViewportClient interface */
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;
	virtual bool InputKey(const FInputKeyEventArgs& EventArgs) override;
	
	/** Determines which texture page was selected. */
	void UpdateSelectedPage(UObject* SelectedObject);

	/** Accessors */
	int32 GetCurrentSelectedPage() const;
	void SetPreviewText(const FText& InPreviewText);
	void SetBackgroundColor(const FColor& InBackgroundColor);
	const FColor& GetBackgroundColor() const;
	void SetForegroundColor(const FColor& InForgroundColor);
	const FColor& GetForegroundColor() const;
	void SetPreviewFontScale(float InScale);
	float GetPreviewFontScale() const;
	void SetDrawFontMetrics(const bool InDrawFontMetrics);
	bool GetDrawFontMetrics() const;
	
	/** Returns the ratio of the size of the font texture to the size of the viewport */
	float GetViewportVerticalScrollBarRatio() const;
	float GetViewportHorizontalScrollBarRatio() const;

	/** FViewport interface */
	virtual float UpdateViewportClientWindowDPIScale() const override;
	virtual bool ShouldDPIScaleSceneCanvas() const override { return false; }

private:
	/** Updates the states of the scrollbars */
	void UpdateScrollBars();

	/** Changes the position of the vertical scrollbar (on a mouse scrollwheel event) */
	void ChangeViewportScrollBarPosition(EScrollDirection Direction);

	/** Returns the positions of the scrollbars relative to the font textures */
	FVector2D GetViewportScrollBarPositions() const;

	/** Get the raw (unscaled) FontEditorViewport size */
	FIntPoint GetDPIUnscaledViewportSizeXY() const;

	void HandleWindowDPIScaleChanged(TSharedRef<SWindow> Window);
private:
	/** Pointer back to the Font viewport control that owns us */
	TWeakPtr<SFontEditorViewport> FontEditorViewportPtr;

	/** Which font page is currently selected */
	int32 CurrentSelectedPage;

	/** Text displayed in font preview viewports */
	FText PreviewText;

	/** Background and foreground color used by font preview viewports */
	FColor BackgroundColor;
	FColor ForegroundColor;

	/** Should we draw the font metrics in the preview? */
	bool bDrawFontMetrics;

	float DrawFontScale;

	/** The size of the gap between pages */
	const int32 PageGap;
};

FFontEditorViewportClient::FFontEditorViewportClient(TWeakPtr<SFontEditorViewport> InFontEditorViewport)
	: FontEditorViewportPtr(InFontEditorViewport)
	, CurrentSelectedPage(INDEX_NONE)
	, DrawFontScale(1.0f)
	, PageGap(4)
{
	PreviewText = LOCTEXT("DefaultPreviewText", "The quick brown fox jumps over the lazy dog");
	BackgroundColor = FColor::Black;
	ForegroundColor = FColor::White;
	bDrawFontMetrics = false;

	FSlateApplication::Get().OnWindowDPIScaleChanged().AddRaw(this, &FFontEditorViewportClient::HandleWindowDPIScaleChanged);
}

FFontEditorViewportClient::~FFontEditorViewportClient()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnWindowDPIScaleChanged().RemoveAll(this);
	}
}

void FFontEditorViewportClient::Draw(FViewport* Viewport, FCanvas* Canvas)
{
	UFont* Font = FontEditorViewportPtr.Pin()->GetFontEditor().Pin()->GetFont();

	if (!FontEditorViewportPtr.Pin()->IsPreviewViewport())
	{
		FVector2f Ratio = FVector2f(GetViewportHorizontalScrollBarRatio(), GetViewportVerticalScrollBarRatio());
		const FIntPoint DPIUnscaledViewportSize = GetDPIUnscaledViewportSizeXY();
		FVector2f ViewportSize = FVector2f((float)DPIUnscaledViewportSize.X, (float)DPIUnscaledViewportSize.Y);
		FVector2f ScrollBarPos = FVector2f(GetViewportScrollBarPositions());
		int32 YOffset = FMath::TruncToInt32((Ratio.Y > 1.0f)? ((ViewportSize.Y - (ViewportSize.Y / Ratio.Y)) * 0.5f): 0);
		int32 YPos = FMath::TruncToInt32(YOffset - ScrollBarPos.Y);
		int32 LastDrawnYPos = YPos;
		int32 XOffset = FMath::TruncToInt32((Ratio.X > 1.0f)? ((ViewportSize.X - (ViewportSize.X / Ratio.X)) * 0.5f): 0);
		int32 XPos = FMath::TruncToInt32(XOffset - ScrollBarPos.X);
	
		UpdateScrollBars();

		Canvas->Clear( FColor(0,0,0));

		// Draw checkerbox background
		const static float BoxSize = 40.0f;
		bool XColor = false;
		bool YColor = false;
		FVector2f BackgroundPos;
		for (BackgroundPos.Y = 0.0f; BackgroundPos.Y <= ViewportSize.Y; BackgroundPos.Y += BoxSize)
		{
			YColor = !YColor;
			XColor = YColor;
			for (BackgroundPos.X = 0.0f; BackgroundPos.X <= ViewportSize.X; BackgroundPos.X += BoxSize)
			{
				XColor = !XColor;
				Canvas->DrawTile( BackgroundPos.X, BackgroundPos.Y, BoxSize, BoxSize, 0.0f, 0.0f, 1.0f, 1.0f, XColor? FLinearColor::Gray: FLinearColor(0.25f, 0.25f, 0.25f));
			}
		}

		// Loop through the pages drawing them if they are visible
		for (int32 Index = 0; Index < Font->Textures.Num(); ++Index)
		{
			UTexture* Texture = Font->Textures[Index];
			// Make sure it's a valid texture page. Could be null if the user
			// is editing things
			if (Texture != NULL)
			{
				// Get the rendering info for this object
				FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Texture);
				// If there is an object configured to handle it, draw the thumbnail
				if (RenderInfo != NULL && RenderInfo->Renderer != NULL)
				{
					uint32 Width, Height;

					// Figure out the size we need
					RenderInfo->Renderer->GetThumbnailSize(Texture, 1.0f, Width, Height);

					// Don't draw if we are outside of our range
					if (YPos + (int32)Height >= 0 && YPos <= (int32)Viewport->GetSizeXY().Y)
					{
						// If hit testing, draw a tile instead
						if (Canvas->IsHitTesting())
						{
							Canvas->SetHitProxy(new HObject(Texture));
						
							// Draw a simple tile
							Canvas->DrawTile( (float)XPos, (float)YPos, (float)Width, (float)Height, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White, nullptr );
						
							Canvas->SetHitProxy(NULL);
						}
						// Otherwise draw the font texture
						else
						{
							// Draw a selected background
							if (Texture->IsSelected())
							{
								Canvas->DrawTile((float)XPos, (float)YPos, (float)Width, (float)Height, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor(0.084f, .127f, 0.098f), nullptr );
							}
							else
							{
								Canvas->DrawTile((float)XPos, (float)YPos, (float)Width, (float)Height, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::Black);
							}

							// Draw the font texture (with alpha blending enabled)
							Canvas->DrawTile((float)XPos, (float)YPos, (float)Width, (float)Height, 0.0f, 0.0f, 1.0f, 1.0f, FLinearColor::White, Texture->GetResource(), true);
						}
					}
					// Update our total height and current draw position
					YPos += Height + PageGap;
				}
			}
		}
	}
	else
	{
		// Erase with our background color
		Canvas->Clear( BackgroundColor);

		static const FVector2D StartPos(4.0f, 4.0f);

		// And draw the text with the foreground color
		if (Font->FontCacheType == EFontCacheType::Runtime)
		{
			const float FontScale = GetDPIScale() * DrawFontScale;
			const float DebugDrawScale = GetDPIScale();

			TSharedRef<FSlateFontCache> FontCache = FSlateApplication::Get().GetRenderer()->GetFontCache();

			FVector2D CurPos = StartPos * FontScale;
			int32 WidestName = 0;

			TArrayView<const FTypefaceEntry> TypefaceEntries;
			if (const FCompositeFont* CompositeFont = Font->GetCompositeFont())
			{
				TypefaceEntries = CompositeFont->DefaultTypeface.Fonts;
			}

			// Draw and measure each name so we can work out where to start drawing the preview text column
			for (const FTypefaceEntry& TypefaceEntry : TypefaceEntries)
			{
				FSlateFontInfo FontInfo(Font, Font->LegacyFontSize, TypefaceEntry.Name);
				FontInfo.FontFallback = EFontFallback::FF_NoFallback;

				FShapedGlyphSequenceRef EntryNameShapedText = FontCache->ShapeBidirectionalText(TypefaceEntry.Name.ToString(), FontInfo, FontScale, TextBiDi::ETextDirection::LeftToRight, GetDefaultTextShapingMethod());
				
				FCanvasShapedTextItem ShapedTextItem(CurPos, EntryNameShapedText, FLinearColor(ForegroundColor));
				Canvas->DrawItem(ShapedTextItem);

				const FVector2D MeasuredText = ShapedTextItem.DrawnSize;
				WidestName = FMath::Max(WidestName, EntryNameShapedText->GetMeasuredWidth());

				CurPos.Y += EntryNameShapedText->GetMaxTextHeight() + (8.0f * FontScale);
			}

			CurPos = FVector2D(WidestName + (12.0f * FontScale), StartPos.Y * FontScale);

			// Draw the preview text using each of the default fonts
			for (const FTypefaceEntry& TypefaceEntry : TypefaceEntries)
			{
				FSlateFontInfo FontInfo(Font, Font->LegacyFontSize, TypefaceEntry.Name);
				FontInfo.FontFallback = EFontFallback::FF_NoFallback;

				FShapedGlyphSequenceRef ShapedPreviewText = FontCache->ShapeBidirectionalText(PreviewText.ToString(), FontInfo, FontScale, TextBiDi::ETextDirection::LeftToRight, GetDefaultTextShapingMethod());

				FCanvasShapedTextItem ShapedTextItem(CurPos, ShapedPreviewText, FLinearColor(ForegroundColor));
				Canvas->DrawItem(ShapedTextItem);

				if (bDrawFontMetrics)
				{
					// Draw the bounding box for the glyphs
					{
						float LineX = 0.0f;
						for (const auto& GlyphToRender : ShapedPreviewText->GetGlyphsToRender())
						{
							if (GlyphToRender.bIsVisible)
							{
								const FShapedGlyphFontAtlasData GlyphAtlasData = FontCache->GetShapedGlyphFontAtlasData(GlyphToRender, FFontOutlineSettings::NoOutline);
								
								const float BitmapRenderScale = GlyphToRender.GetBitmapRenderScale();
								const float InvBitmapRenderScale = 1.0f / BitmapRenderScale;

								const double X = CurPos.X + LineX + GlyphAtlasData.HorizontalOffset + GlyphToRender.XOffset;
								const double Y = CurPos.Y - GlyphAtlasData.VerticalOffset + GlyphToRender.YOffset + ((ShapedPreviewText->GetTextBaseline() + ShapedPreviewText->GetMaxTextHeight()) * InvBitmapRenderScale);

								FCanvasBoxItem BoundingBoxItem(FVector2D(X, Y), FVector2D(GlyphAtlasData.USize * BitmapRenderScale, GlyphAtlasData.VSize * BitmapRenderScale));
								BoundingBoxItem.SetColor(FColor::Orange);
								Canvas->DrawItem(BoundingBoxItem);
							}

							LineX += GlyphToRender.XAdvance;
						}
					}

					// Draw the bounding box for the grapheme clusters
					{
						float LineX = 0.0f;
						const auto& GlyphsToRender = ShapedPreviewText->GetGlyphsToRender();
						for (int32 CurrentGlyphIndex = 0; CurrentGlyphIndex < GlyphsToRender.Num(); ++CurrentGlyphIndex)
						{
							// A single character may produce multiple glyphs which must be treated as a single logic unit
							int16 GlyphClusterAdvance = 0;
							FBox2D GlyphClusterBounds = FBox2D(ForceInitToZero);
							for (;;)
							{
								const auto& GlyphToRender = GlyphsToRender[CurrentGlyphIndex];
								const bool bIsWithinGlyphCluster = GlyphsToRender.IsValidIndex(CurrentGlyphIndex + 1) && GlyphToRender.SourceIndex == GlyphsToRender[CurrentGlyphIndex + 1].SourceIndex;

								if (GlyphToRender.bIsVisible)
								{
									const FShapedGlyphFontAtlasData GlyphAtlasData = FontCache->GetShapedGlyphFontAtlasData(GlyphToRender,FFontOutlineSettings::NoOutline);
									
									const float BitmapRenderScale = GlyphToRender.GetBitmapRenderScale();
									const float InvBitmapRenderScale = 1.0f / BitmapRenderScale;

									const double X = CurPos.X + LineX + GlyphClusterAdvance + GlyphAtlasData.HorizontalOffset + GlyphToRender.XOffset;
									const double Y = CurPos.Y - GlyphAtlasData.VerticalOffset + GlyphToRender.YOffset + ((ShapedPreviewText->GetTextBaseline() + ShapedPreviewText->GetMaxTextHeight()) * InvBitmapRenderScale);

									FVector2D ExtraWidth = FVector2D(ForceInitToZero);
									if (GlyphClusterBounds.bIsValid)
									{
										ExtraWidth.X = (GlyphClusterBounds.Min.X > X) ? FMath::Abs(GlyphClusterBounds.Min.X - X) : 0.0f;
										ExtraWidth.Y = (GlyphClusterBounds.Min.Y > Y) ? FMath::Abs(GlyphClusterBounds.Min.Y - Y) : 0.0f;
									}

									GlyphClusterBounds += FBox2D(FVector2D(X, Y), FVector2D(GlyphAtlasData.USize * BitmapRenderScale, GlyphAtlasData.VSize * BitmapRenderScale));
									GlyphClusterBounds.Max += ExtraWidth;
								}

								GlyphClusterAdvance += GlyphToRender.XAdvance;

								if (!bIsWithinGlyphCluster)
								{
									break;
								}

								++CurrentGlyphIndex;
							}

							FCanvasBoxItem BoundingBoxItem(GlyphClusterBounds.Min, GlyphClusterBounds.Max);
							BoundingBoxItem.SetColor(FLinearColor::Yellow);
							Canvas->DrawItem(BoundingBoxItem);

							LineX += GlyphClusterAdvance;
						}
					}

					// Draw the baseline
					{
						const double Y = CurPos.Y + ShapedPreviewText->GetTextBaseline() + ShapedPreviewText->GetMaxTextHeight();

						FCanvasLineItem BaseLineItem(FVector2D(CurPos.X, Y), FVector2D(CurPos.X + ShapedPreviewText->GetMeasuredWidth(), Y));
						BaseLineItem.SetColor(FLinearColor::Red);
						Canvas->DrawItem(BaseLineItem);
					}

					// Draw the bounding box for the line height
					{
						FCanvasBoxItem LineHeightBoxItem(CurPos, FVector2D(ShapedPreviewText->GetMeasuredWidth(), ShapedPreviewText->GetMaxTextHeight()));
						LineHeightBoxItem.SetColor(FLinearColor::Green);
						Canvas->DrawItem(LineHeightBoxItem);
					}
				}

				CurPos.Y += ShapedPreviewText->GetMaxTextHeight() + (8.0f * FontScale);
			}

			// Draw the key
			if (bDrawFontMetrics)
			{
				const FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");
				const float KeyBoxSize = 14.0f * DebugDrawScale;

				struct FKeyDataType
				{
					FLinearColor KeyColor;
					FText KeyText;
				};

				static const FKeyDataType KeyDataArray[] = {
					{ FLinearColor::Red,	LOCTEXT("BaselineKeyLabel", "Baseline") },
					{ FLinearColor::Green,	LOCTEXT("LineBoundsKeyLabel", "Line Bounds") },
					{ FLinearColor::Yellow,	LOCTEXT("GraphemeClusterBoundsKeyLabel", "Grapheme Cluster Bounds") },
					{ FColor::Orange,		LOCTEXT("GlyphBoundsKeyLabel", "Glyph Bounds") },
				};

				for (const FKeyDataType& KeyData : KeyDataArray)
				{
					FCanvasBoxItem KeyBox(CurPos + (KeyBoxSize * 0.25f), FVector2D(KeyBoxSize * 0.5f, KeyBoxSize * 0.5f));
					KeyBox.SetColor(KeyData.KeyColor);
					KeyBox.LineThickness = KeyBoxSize * 0.5f;
					Canvas->DrawItem(KeyBox);

					CurPos.X += KeyBoxSize + (4.0f * DebugDrawScale);

					FShapedGlyphSequenceRef KeyLabelShapedText = FontCache->ShapeBidirectionalText(*KeyData.KeyText.ToString(), FontInfo, DebugDrawScale, TextBiDi::ETextDirection::LeftToRight, GetDefaultTextShapingMethod());
					FCanvasShapedTextItem ShapedTextItem(CurPos, KeyLabelShapedText, FLinearColor(ForegroundColor));
					Canvas->DrawItem(ShapedTextItem);

					CurPos.X += KeyLabelShapedText->GetMeasuredWidth() + (8.0f * DebugDrawScale);
				}
			}
		}
		else
		{
			FCanvasTextItem TextItem(StartPos, PreviewText, Font, FLinearColor(ForegroundColor));
			TextItem.BlendMode = (Font->ImportOptions.bUseDistanceFieldAlpha) ? SE_BLEND_TranslucentDistanceField : SE_BLEND_Translucent;
			Canvas->DrawItem(TextItem);
		}
	}
}

bool FFontEditorViewportClient::InputKey(const FInputKeyEventArgs& EventArgs)
{
	// The preview viewport doesn't need to process key input
	if (FontEditorViewportPtr.Pin()->IsPreviewViewport())
	{
		return false;
	}

	bool bHandled = false;

	const FKey& Key = EventArgs.Key;
	FViewport* Viewport = EventArgs.Viewport;

	if (Key == EKeys::LeftMouseButton && EventArgs.Event == IE_Released)
	{
		EventArgs.Viewport->Invalidate();
		bHandled = true;
	}
	else
	{
		if((Key == EKeys::LeftMouseButton || Key == EKeys::RightMouseButton) && EventArgs.Event == IE_Pressed)
		{
			const int32 HitX = Viewport->GetMouseX();
			const int32 HitY = Viewport->GetMouseY();
			// See if we hit something
			HHitProxy* HitResult = Viewport->GetHitProxy(HitX,HitY);
			if (HitResult)
			{
				if (HitResult->IsA(HObject::StaticGetType()))
				{
					// Get the object that was hit
					UObject* HitObject = ((HObject*)HitResult)->Object;
					if (HitObject)
					{
						// Turn off all others and set it as selected
						if (HitObject->IsA(AActor::StaticClass()))
						{
							//UE_LOGF(LogFontPropDlg, Warning, "WxFontPropertiesDlg::InputKey : selecting actor!");
							GEditor->GetSelectedActors()->DeselectAll();
							GEditor->GetSelectedActors()->Select(HitObject);
						}
						else
						{
							//UE_LOGF(LogFontPropDlg, Log, "WxFontPropertiesDlg::InputKey : selecting object!");
							GEditor->GetSelectedObjects()->DeselectAll();
							GEditor->GetSelectedObjects()->Select(HitObject);
						}
						// Update our internal state for selected page
						// buttons, etc.
						UpdateSelectedPage(HitObject);
					}
				}
			}
		
			// Force a redraw
			Viewport->Invalidate();

			bHandled = true;
		}
		// Did they scroll using the mouse wheel?
		else if (Key == EKeys::MouseScrollUp)
		{
			ChangeViewportScrollBarPosition(Scroll_Down);
			bHandled = true;
		}
		// Did they scroll using the mouse wheel?
		else if (Key == EKeys::MouseScrollDown)
		{
			ChangeViewportScrollBarPosition(Scroll_Up);
			bHandled = true;
		}
	}
	
	return bHandled;
}

void FFontEditorViewportClient::UpdateSelectedPage(UObject* SelectedObject)
{
	UFont* Font = FontEditorViewportPtr.Pin()->GetFontEditor().Pin()->GetFont();

	// Default to non-selected
	CurrentSelectedPage = INDEX_NONE;

	// Search through the font's texture array seeing if this is a match
	for (int32 Index = 0; Index < Font->Textures.Num() && CurrentSelectedPage == INDEX_NONE; Index++)
	{
		// See if the pointers match
		if (Font->Textures[Index] == SelectedObject)
		{
			CurrentSelectedPage = Index;
			break;
		}
	}

	FontEditorViewportPtr.Pin()->GetFontEditor().Pin()->SetSelectedPage(CurrentSelectedPage);
}

int32 FFontEditorViewportClient::GetCurrentSelectedPage() const
{
	return CurrentSelectedPage;
}

void FFontEditorViewportClient::SetPreviewText(const FText& InPreviewText)
{
	PreviewText = InPreviewText;
}

void FFontEditorViewportClient::SetBackgroundColor(const FColor& InBackgroundColor)
{
	BackgroundColor = InBackgroundColor;
}

const FColor& FFontEditorViewportClient::GetBackgroundColor() const
{
	return BackgroundColor;
}

void FFontEditorViewportClient::SetForegroundColor(const FColor& InForegroundColor)
{
	ForegroundColor = InForegroundColor;
}

const FColor& FFontEditorViewportClient::GetForegroundColor() const
{
	return ForegroundColor;
}

void FFontEditorViewportClient::SetPreviewFontScale(float InScale)
{
	DrawFontScale = InScale;
}

float FFontEditorViewportClient::GetPreviewFontScale() const
{
	return DrawFontScale;
}

void FFontEditorViewportClient::SetDrawFontMetrics(const bool InDrawFontMetrics)
{
	bDrawFontMetrics = InDrawFontMetrics;
}

bool FFontEditorViewportClient::GetDrawFontMetrics() const
{
	return bDrawFontMetrics;
}

float FFontEditorViewportClient::GetViewportVerticalScrollBarRatio() const
{
	float WidgetHeight = 1.0f;
	float TextureHeight = 1.0f;
	if (FontEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		UFont* Font = FontEditorViewportPtr.Pin()->GetFontEditor().Pin()->GetFont();
		
		WidgetHeight = (float)GetDPIUnscaledViewportSizeXY().Y;
		
		for (int32 Idx = 0; Idx < Font->Textures.Num(); ++Idx)
		{
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Font->Textures[Idx]);
			if (RenderInfo != NULL && RenderInfo->Renderer != NULL)
			{
				uint32 Width, Height;
				RenderInfo->Renderer->GetThumbnailSize(Font->Textures[Idx], 1.0f, Width, Height);
				TextureHeight += Height;
			}
		}

		TextureHeight += (Font->Textures.Num() - 1) * PageGap;
	}

	return WidgetHeight / TextureHeight;
}

float FFontEditorViewportClient::GetViewportHorizontalScrollBarRatio() const
{
	uint32 Width = 1;
	float WidgetWidth = 1.0f;
	if (FontEditorViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		UFont* Font = FontEditorViewportPtr.Pin()->GetFontEditor().Pin()->GetFont();
		uint32 Height = 1;

		WidgetWidth = (float)GetDPIUnscaledViewportSizeXY().X;

		if(Font->Textures.Num())
		{
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Font->Textures[0]);
			if (RenderInfo != NULL && RenderInfo->Renderer != NULL)
			{
				RenderInfo->Renderer->GetThumbnailSize(Font->Textures[0], 1.0f, Width, Height);
			}
		}

	}

	return WidgetWidth / Width;
}

FIntPoint FFontEditorViewportClient::GetDPIUnscaledViewportSizeXY() const
{
	const FIntPoint SizeXY = FontEditorViewportPtr.Pin()->GetViewport()->GetSizeXY();
	const float DPIScale = GetDPIScale();
	return FIntPoint(FMath::TruncToInt32(SizeXY.X / DPIScale), FMath::TruncToInt32(SizeXY.Y / DPIScale));
}

float FFontEditorViewportClient::UpdateViewportClientWindowDPIScale() const
{
	float DPIScale = 1.f;
	if (FontEditorViewportPtr.IsValid())
	{
		TSharedPtr<SWindow> WidgetWindow = FSlateApplication::Get().FindWidgetWindow(FontEditorViewportPtr.Pin().ToSharedRef());
		if (WidgetWindow.IsValid())
		{
			DPIScale = WidgetWindow->GetNativeWindow()->GetDPIScaleFactor();
		}
	}

	return DPIScale;
}

void FFontEditorViewportClient::UpdateScrollBars()
{
	if (FontEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid() && FontEditorViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		float VRatio = GetViewportVerticalScrollBarRatio();
		float HRatio = GetViewportHorizontalScrollBarRatio();
		float VDistFromBottom = FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float HDistFromBottom = FontEditorViewportPtr.Pin()->GetHorizontalScrollBar()->DistanceFromBottom();

		if (VRatio < 1.0f)
		{
			if (VDistFromBottom < 1.0f)
			{
				FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->SetState(FMath::Clamp(1.0f - VRatio - VDistFromBottom, 0.0f, 1.0f), VRatio);
			}
			else
			{
				FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->SetState(0.0f, VRatio);
			}
		}

		if (HRatio < 1.0f)
		{
			if (HDistFromBottom < 1.0f)
			{
				FontEditorViewportPtr.Pin()->GetHorizontalScrollBar()->SetState(FMath::Clamp(1.0f - HRatio - HDistFromBottom, 0.0f, 1.0f), HRatio);
			}
			else
			{
				FontEditorViewportPtr.Pin()->GetHorizontalScrollBar()->SetState(0.0f, HRatio);
			}
		}
	}
}

void FFontEditorViewportClient::ChangeViewportScrollBarPosition(EScrollDirection Direction)
{
	if (FontEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid())
	{
		float Ratio = GetViewportVerticalScrollBarRatio();
		float DistFromBottom = FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float OneMinusRatio = 1.0f - Ratio;
		float Diff = 0.1f * OneMinusRatio;

		if (Direction == Scroll_Down)
		{
			Diff *= -1.0f;
		}

		FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->SetState(FMath::Clamp(OneMinusRatio - DistFromBottom + Diff, 0.0f, OneMinusRatio), Ratio);

		FontEditorViewportPtr.Pin()->GetViewport()->Invalidate();
		FontEditorViewportPtr.Pin()->GetViewport()->InvalidateDisplay();
	}
}

FVector2D FFontEditorViewportClient::GetViewportScrollBarPositions() const
{
	FVector2D Positions = FVector2D::ZeroVector;
	if (FontEditorViewportPtr.Pin()->GetVerticalScrollBar().IsValid() && FontEditorViewportPtr.Pin()->GetHorizontalScrollBar().IsValid())
	{
		UFont* Font = FontEditorViewportPtr.Pin()->GetFontEditor().Pin()->GetFont();
		float VRatio = GetViewportVerticalScrollBarRatio();
		float HRatio = GetViewportHorizontalScrollBarRatio();
		float VDistFromBottom = FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->DistanceFromBottom();
		float HDistFromBottom = FontEditorViewportPtr.Pin()->GetHorizontalScrollBar()->DistanceFromBottom();
	
		if ((FontEditorViewportPtr.Pin()->GetVerticalScrollBar()->GetVisibility() == EVisibility::Visible) && VDistFromBottom < 1.0f)
		{
			float TextureHeight = 0.0f;
			for (int32 Idx = 0; Idx < Font->Textures.Num(); ++Idx)
			{
				FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Font->Textures[Idx]);
				if (RenderInfo != NULL && RenderInfo->Renderer != NULL)
				{
					uint32 Width, Height;
					RenderInfo->Renderer->GetThumbnailSize(Font->Textures[Idx], 1.0f, Width, Height);
					TextureHeight += Height;
				}
			}

			TextureHeight += (Font->Textures.Num() - 1) * PageGap;

			Positions.Y = FMath::Clamp(1.0f - VRatio - VDistFromBottom, 0.0f, 1.0f) * TextureHeight;
		}
		else
		{
			Positions.Y = 0.0f;
		}

		if ((FontEditorViewportPtr.Pin()->GetHorizontalScrollBar()->GetVisibility() == EVisibility::Visible) && HDistFromBottom < 1.0f)
		{
			uint32 Width = 0;
			uint32 Height = 0;
			FThumbnailRenderingInfo* RenderInfo = GUnrealEd->GetThumbnailManager()->GetRenderingInfo(Font->Textures[0]);
			if (RenderInfo != NULL && RenderInfo->Renderer != NULL)
			{
				RenderInfo->Renderer->GetThumbnailSize(Font->Textures[0], 1.0f, Width, Height);
			}

			Positions.X = FMath::Clamp(1.0f - HRatio - HDistFromBottom, 0.0f, 1.0f) * Width;
		}
		else
		{
			Positions.X = 0.0f;
		}
	}

	return Positions;
}


void FFontEditorViewportClient::HandleWindowDPIScaleChanged(TSharedRef<SWindow> Window)
{
	RequestUpdateDPIScale();
	if (FontEditorViewportPtr.IsValid())
	{
		FontEditorViewportPtr.Pin()->RefreshViewport();
	}
}

SFontEditorViewport::~SFontEditorViewport()
{

}

void SFontEditorViewport::Construct(const FArguments& InArgs)
{
	FontEditorPtr = InArgs._FontEditor;
	bIsPreview = InArgs._IsPreview;

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.FillHeight(1)
				[
					SAssignNew(ViewportWidget, SViewport)
					.EnableGammaCorrection(false)
					.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
					.ShowEffectWhenDisabled(false)
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(FontViewportVerticalScrollBar, SScrollBar)
				.Visibility(this, &SFontEditorViewport::GetViewportVerticalScrollBarVisibility)
				.OnUserScrolled(this, &SFontEditorViewport::OnViewportVerticalScrollBarScrolled)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(FontViewportHorizontalScrollBar, SScrollBar)
			.Orientation( Orient_Horizontal )
			.Visibility(this, &SFontEditorViewport::GetViewportHorizontalScrollBarVisibility)
			.OnUserScrolled(this, &SFontEditorViewport::OnViewportHorizontalScrollBarScrolled)
		]
	];

	ViewportClient = MakeShareable(new FFontEditorViewportClient(SharedThis(this)));

	Viewport = FSceneViewport::Create(ViewportClient, ViewportWidget);
}

void SFontEditorViewport::RefreshViewport()
{
	Viewport->Invalidate();
}

int32 SFontEditorViewport::GetCurrentSelectedPage() const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetCurrentSelectedPage();
	}

	return INDEX_NONE;
}

void SFontEditorViewport::SetPreviewText(const FText& PreviewText)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetPreviewText(PreviewText);

		RefreshViewport();
	}
}

void SFontEditorViewport::SetPreviewBackgroundColor(const FColor& BackgroundColor)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetBackgroundColor(BackgroundColor);

		RefreshViewport();
	}
}

const FColor& SFontEditorViewport::GetPreviewBackgroundColor() const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetBackgroundColor();
	}

	return FColor::Black;
}

void SFontEditorViewport::SetPreviewForegroundColor(const FColor& InForegroundColor)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetForegroundColor(InForegroundColor);

		RefreshViewport();
	}
}

const FColor& SFontEditorViewport::GetPreviewForegroundColor() const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetForegroundColor();
	}

	return FColor::White;
}

void SFontEditorViewport::SetPreviewFontScale(float InScale)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetPreviewFontScale(InScale);

		RefreshViewport();
	}
}

float SFontEditorViewport::GetPreviewFontScale() const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetPreviewFontScale();
	}

	return 1.0f;
}

void SFontEditorViewport::SetPreviewFontMetrics(const bool InDrawFontMetrics)
{
	if (ViewportClient.IsValid())
	{
		ViewportClient->SetDrawFontMetrics(InDrawFontMetrics);

		RefreshViewport();
	}
}

bool SFontEditorViewport::GetPreviewFontMetrics() const
{
	if (ViewportClient.IsValid())
	{
		return ViewportClient->GetDrawFontMetrics();
	}

	return false;
}

TWeakPtr<IFontEditor> SFontEditorViewport::GetFontEditor() const
{
	return FontEditorPtr;
}

bool SFontEditorViewport::IsPreviewViewport() const
{
	return bIsPreview;
}

TSharedPtr<FSceneViewport> SFontEditorViewport::GetViewport() const
{
	return Viewport;
}

TSharedPtr<SViewport> SFontEditorViewport::GetViewportWidget() const
{
	return ViewportWidget;
}

TSharedPtr<SScrollBar> SFontEditorViewport::GetVerticalScrollBar() const
{
	return FontViewportVerticalScrollBar;
}

TSharedPtr<SScrollBar> SFontEditorViewport::GetHorizontalScrollBar() const
{
	return FontViewportHorizontalScrollBar;
}

EVisibility SFontEditorViewport::GetViewportVerticalScrollBarVisibility() const
{
	return (!bIsPreview && (ViewportClient->GetViewportVerticalScrollBarRatio() < 1.0f))? EVisibility::Visible: EVisibility::Collapsed;
}

EVisibility SFontEditorViewport::GetViewportHorizontalScrollBarVisibility() const
{
	return (!bIsPreview && (ViewportClient->GetViewportHorizontalScrollBarRatio() < 1.0f))? EVisibility::Visible: EVisibility::Collapsed;
}

void SFontEditorViewport::OnViewportVerticalScrollBarScrolled(float InScrollOffsetFraction)
{
	float Ratio = ViewportClient->GetViewportVerticalScrollBarRatio();
	float MaxOffset = (Ratio < 1.0f)? 1.0f - Ratio: 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);
	FontViewportVerticalScrollBar->SetState(InScrollOffsetFraction, Ratio);
	RefreshViewport();
}

void SFontEditorViewport::OnViewportHorizontalScrollBarScrolled(float InScrollOffsetFraction)
{
	float Ratio = ViewportClient->GetViewportHorizontalScrollBarRatio();
	float MaxOffset = (Ratio < 1.0f)? 1.0f - Ratio: 0.0f;
	InScrollOffsetFraction = FMath::Clamp(InScrollOffsetFraction, 0.0f, MaxOffset);
	FontViewportHorizontalScrollBar->SetState(InScrollOffsetFraction, Ratio);
	RefreshViewport();
}

#undef LOCTEXT_NAMESPACE
