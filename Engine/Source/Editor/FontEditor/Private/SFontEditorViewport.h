// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FFontEditorViewportClient;
class FSceneViewport;
class FText;
class IFontEditor;
class SScrollBar;
class SViewport;
struct FColor;

/*-----------------------------------------------------------------------------
   SFontEditorViewport
-----------------------------------------------------------------------------*/

class SFontEditorViewport : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SFontEditorViewport )
		: _IsPreview(false)
		{}

		SLATE_ARGUMENT(TWeakPtr<IFontEditor>, FontEditor)
		SLATE_ARGUMENT(bool, IsPreview)
	SLATE_END_ARGS()

	/** Destructor */
	~SFontEditorViewport();

	/** SCompoundWidget interface */
	void Construct(const FArguments& InArgs);
	
	/** Refreshes the viewport */
	void RefreshViewport();

	/** Accessors */
	int32 GetCurrentSelectedPage() const;
	void SetPreviewText(const FText& PreviewText);
	void SetPreviewBackgroundColor(const FColor& BackgroundColor);
	const FColor& GetPreviewBackgroundColor() const;
	void SetPreviewForegroundColor(const FColor& ForgroundColor);
	const FColor& GetPreviewForegroundColor() const;
	void SetPreviewFontScale(float InScale);
	float GetPreviewFontScale() const;
	void SetPreviewFontMetrics(const bool InDrawFontMetrics);
	bool GetPreviewFontMetrics() const;
	TWeakPtr<IFontEditor> GetFontEditor() const;
	bool IsPreviewViewport() const;
	TSharedPtr<FSceneViewport> GetViewport() const;
	TSharedPtr<SViewport> GetViewportWidget() const;
	TSharedPtr<SScrollBar> GetVerticalScrollBar() const;
	TSharedPtr<SScrollBar> GetHorizontalScrollBar() const;

protected:
	/** Returns the visibility of the viewport scrollbars */
	EVisibility GetViewportVerticalScrollBarVisibility() const;
	EVisibility GetViewportHorizontalScrollBarVisibility() const;

	/** Called when the viewport scrollbars are scrolled */
	void OnViewportVerticalScrollBarScrolled(float InScrollOffsetFraction);
	void OnViewportHorizontalScrollBarScrolled(float InScrollOffsetFraction);

private:
	/** Pointer back to the Font editor tool that owns us */
	TWeakPtr<IFontEditor> FontEditorPtr;

	/** If true, this is a viewport for the font editor's preview tab */
	bool bIsPreview;
	
	/** Level viewport client */
	TSharedPtr<FFontEditorViewportClient> ViewportClient;

	/** Slate viewport for rendering and I/O */
	TSharedPtr<FSceneViewport> Viewport;

	/** Viewport widget*/
	TSharedPtr<SViewport> ViewportWidget;

	/** Vertical scrollbar */
	TSharedPtr<SScrollBar> FontViewportVerticalScrollBar;

	/** Horizontal scrollbar */
	TSharedPtr<SScrollBar> FontViewportHorizontalScrollBar;
};
