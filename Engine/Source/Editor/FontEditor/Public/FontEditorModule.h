// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/IToolkit.h"
#include "UObject/NameTypes.h"

class IFontEditor;
class IFontFaceEditor;
class IToolkitHost;
class UFont;
class UFontFace;

extern const FName FontEditorAppIdentifier;


/*-----------------------------------------------------------------------------
   IFontEditorModule
-----------------------------------------------------------------------------*/

class IFontEditorModule : public IModuleInterface,
	public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/** Creates a new Font editor */
	virtual TSharedRef<IFontEditor> CreateFontEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UFont* Font ) = 0;

	/** Creates a new Font Face editor */
	virtual TSharedRef<IFontFaceEditor> CreateFontFaceEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UFontFace* FontFace ) = 0;
};
