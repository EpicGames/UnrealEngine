// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

class UFont;

/*-----------------------------------------------------------------------------
   IFontEditor
-----------------------------------------------------------------------------*/

class IFontEditor : public FAssetEditorToolkit
{
public:
	/** Returns the font asset being inspected by the font editor */
	virtual UFont* GetFont() const = 0;

	/** Assigns a font texture object to the page properties control when a new page is selected */
	virtual void SetSelectedPage(int32 PageIdx) = 0;

	/** Refresh the preview viewport */
	virtual void RefreshPreview() = 0;
};


