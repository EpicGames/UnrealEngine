// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"

class UFontFace;

/*-----------------------------------------------------------------------------
   IFontFaceEditor
-----------------------------------------------------------------------------*/

class IFontFaceEditor : public FAssetEditorToolkit
{
public:
	/** Returns the font face asset being inspected by the font face editor */
	virtual UFontFace* GetFontFace() const = 0;

	/** Refresh the preview viewport */
	virtual void RefreshPreview() = 0;
};
